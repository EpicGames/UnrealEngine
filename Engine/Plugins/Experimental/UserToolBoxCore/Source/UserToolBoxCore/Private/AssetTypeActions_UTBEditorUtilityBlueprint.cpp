// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions_UTBEditorUtilityBlueprint.h"
#include "GlobalEditorUtilityBase.h"
#include "IContentBrowserSingleton.h"
#include "UserToolBoxSubsystem.h"
#include "UTBEditorBlueprint.h"
#define LOCTEXT_NAMESPACE "AssetTypeActions"

/////////////////////////////////////////////////////
// FAssetTypeActions_EditorUtilityBlueprint

FText FAssetTypeActions_UTBEditorBlueprint::GetName() const
{
	return FText::FromString("User ToolBox Command Blueprint");
}

FColor FAssetTypeActions_UTBEditorBlueprint::GetTypeColor() const
{
	return FColor(0, 169, 0);
}

UClass* FAssetTypeActions_UTBEditorBlueprint::GetSupportedClass() const
{
	return UUTBEditorBlueprint::StaticClass();
}

bool FAssetTypeActions_UTBEditorBlueprint::HasActions(const TArray<UObject*>& InObjects) const
{
	return false;
}
uint32 FAssetTypeActions_UTBEditorBlueprint::GetCategories()
{
	return UUserToolboxSubsystem::AssetCategory;
}




#undef LOCTEXT_NAMESPACE
