// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeActions/AssetTypeActions_Blueprint.h"


class FAssetTypeActions_UTBEditorBlueprint : public FAssetTypeActions_Blueprint
{
public:
	// IAssetTypeActions interface
	virtual FText GetName() const override;
	virtual FColor GetTypeColor() const override;
	virtual UClass* GetSupportedClass() const override;
	virtual bool HasActions(const TArray<UObject*>& InObjects) const override;
	virtual uint32 GetCategories() override;
	virtual bool CanLocalize() const override { return false; }
	// End of IAssetTypeActions interface
};
