// Copyright Epic Games, Inc. All Rights Reserved.


#include "BaseCommandCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "UserToolBoxSubsystem.h"
#include "UTBBaseCommand.h"
#include "Widgets/Input/SButton.h"
#include "Editor.h"
#include "Widgets/Images/SImage.h"

void FBaseCommandCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedRef<IPropertyHandle> PropertyHandle= DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUTBBaseCommand,IconPath));
	TArray<UObject*> Outers;
	PropertyHandle->GetOuterObjects(Outers);
	IDetailPropertyRow* PropertyRow=DetailBuilder.EditDefaultProperty(PropertyHandle);
	if (PropertyRow==nullptr)
	{
		return;
	}
		
	DetailBuilder.EditDefaultProperty(PropertyHandle)->CustomWidget()
	.ValueWidget
	[
		SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				PropertyHandle->CreatePropertyValueWidget()
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>( "Button" ))
				.ContentPadding(1.0f)
				.OnClicked_Lambda([Outers]()
				{
					
					if (FString IconValue=""; GEditor->GetEditorSubsystem<UUserToolboxSubsystem>()->PickAnIcon(IconValue))
					{
						for (UObject* Object:Outers)
						{
							
							if (UUTBBaseCommand* Command=Cast<UUTBBaseCommand>(Object);IsValid(Command))
							{
								Command->IconPath=IconValue;
								Command->PostEditChange();
							}
						}
						
						
					}
					return FReply::Handled();
				})
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("ShowFlagsMenu.SubMenu.Sprites"))
				]
			]
	]
	.NameWidget
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	;
	
	
}
FBaseCommandCustomization::~FBaseCommandCustomization()
{
}
void FBaseCommandCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CustomizeDetails(*DetailBuilder);
}
