// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

/**
 * 
 */




class FBaseCommandCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FBaseCommandCustomization());
	}
	virtual ~FBaseCommandCustomization() override;

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
};




