// Copyright Epic Games, Inc. All Rights Reserved.


#include "Editor/SCommandPickMenu.h"

#include "UserToolBoxBaseBlueprint.h"
#include "UTBEditorBlueprint.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor/UTBTabEditor.h"

#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Text/STextBlock.h"
#include "UObject/UObjectIterator.h"

SCommandPickMenuWidget::SCommandPickMenuWidget()
{
}

SCommandPickMenuWidget::~SCommandPickMenuWidget()
{
}

void SCommandPickMenuWidget::Construct(const FArguments& InArgs)
{
	OnCommandSelectionChanged=InArgs._OnCommandSelectionChanged;
	ReconstructCommandTree();
	
	
	SAssignNew(TreeViewPtr,STreeView<TSharedPtr<FCommandTreeNode>>)
		.AllowOverscroll(EAllowOverscroll::Yes)
		.TreeItemsSource(&CommandTree)
		.OnSelectionChanged_Lambda([this](TSharedPtr<FCommandTreeNode> Node, ESelectInfo::Type)
		{
			if (!Node.IsValid())
			{
				Selection=nullptr;
				OnCommandSelectionChanged.ExecuteIfBound(nullptr);
				return;
			}
			if (!Node->bIsCategory)
			{
				Selection=Node->Class;
				
				OnCommandSelectionChanged.ExecuteIfBound(Node->Class);
				
				
			}
			else
			{
				Selection=nullptr;
				OnCommandSelectionChanged.ExecuteIfBound(nullptr);
			}	
		})
		.OnContextMenuOpening(FOnContextMenuOpening::CreateLambda([this]()
		{
			if (Selection!=nullptr)
			{
				return SNew(STextBlock).Text(FText::FromString(Selection->GetFullName()));
			}
			return SNew(STextBlock).Text(FText::FromString("Nullptr"));
			
		}))
		.OnSetExpansionRecursive(this, &SCommandPickMenuWidget::OnSetExpansionRecursive)
		.OnGenerateRow(STreeView<TSharedPtr<FCommandTreeNode>>::FOnGenerateRow::CreateLambda([](TSharedPtr<FCommandTreeNode> Node,const TSharedRef<STableViewBase>& OwnerTable)
		{
			TSharedPtr<STextBlock> TextBlock;
			if (Node.IsValid())
			{
				if (Node->bIsCategory)
				{
					SAssignNew(TextBlock,STextBlock).Text(FText::FromString(Node->Name))
					.Font( FCoreStyle::GetDefaultFontStyle("Bold", 9) );	
				}
				else
				{
					SAssignNew(TextBlock,STextBlock).Text(FText::FromString(Node->Name))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
					.ToolTipText_Lambda([Node]()
					{
						UUTBBaseCommand* Command=Cast<UUTBBaseCommand>(Node->Class->GetDefaultObject());
						if (IsValid(Command))
						{
							return FText::FromString(Command->Tooltip);	
						}else
						{
							return FText::FromString("Tooltip not available");
						}
												
					})
					
					;
					
				}
				

			}
				
			else
			{
				SAssignNew(TextBlock,STextBlock).Text(FText::FromString("Invalid node"));
			}
			
			
			return SNew(STableRow<TSharedPtr<FCommandTreeNode>>,OwnerTable).OnDragDetected(FOnDragDetected::CreateLambda([Node](const FGeometry& InGeometry, const FPointerEvent& InPointerEvent )
			{
				if (Node->bIsCategory)
				{
					return FReply::Unhandled();
				}
				return FReply::Handled().BeginDragDrop(FCommandDragDropOperation::New(Cast<UUTBBaseCommand>(Node->Class->GetDefaultObject())));	
			}))
			[
				TextBlock.ToSharedRef()
			];
		}))
		.OnGetChildren(STreeView<TSharedPtr<FCommandTreeNode>>::FOnGetChildren::CreateLambda([this](TSharedPtr<FCommandTreeNode> ParentNode,TArray<TSharedPtr<FCommandTreeNode>>& Children)
		{
			
			
			Children=ParentNode->FilteredChildren;

		}));
		

	this->ChildSlot
	[
		SNew(SVerticalBox)
		
		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(EVerticalAlignment::VAlign_Top)
		[
			SNew(SSearchBox)
			.OnTextChanged(FOnTextChanged::CreateLambda([this](const FText& Text)
			{
				this->UpdateFilteredTreeNodes(Text.ToString());
			}))
		]
		+SVerticalBox::Slot()
		[
			TreeViewPtr.ToSharedRef()
		]
		
	];
	UpdateFilteredTreeNodes("");
}

void SCommandPickMenuWidget::ReconstructCommandTree()
{
	CommandTree.Empty();
	CommandTreeAllNodes.Empty();
	LeafsWithParentHierarchy.Empty();

	/*
	 * Need o load blueprint command before
	 */
	
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	FARFilter Filter;
	Filter.bIncludeOnlyOnDiskAssets = false;
	Filter.bRecursivePaths = true;
	Filter.bRecursiveClasses = true;
	Filter.ClassPaths = {UUTBEditorBlueprint::StaticClass()->GetClassPathName()};
	TArray<FAssetData> CommandBlueprintAssetDatas;
	AssetRegistry.GetAssets(Filter,CommandBlueprintAssetDatas);
	for (FAssetData AssetData:CommandBlueprintAssetDatas)
	{
		AssetData.GetAsset();
	}
	TArray<UClass*> Classes;
	for( TObjectIterator<UClass> It ; It ; ++It )
	{
		UClass* CurrentClass=*It;
		if (CurrentClass->IsChildOf(UUTBBaseCommand::StaticClass()))
		{
			if (CurrentClass->GetName().StartsWith(TEXT("SKEL_")) || CurrentClass->GetName().StartsWith(TEXT("REINST_")))
			{
				continue;
			}
			if ( (CurrentClass)!= UUTBBaseCommand::StaticClass() && (CurrentClass)!=UUTBEditorBlueprint::StaticClass() && (CurrentClass)!=UUserToolBoxBaseBlueprint::StaticClass() && !CurrentClass->HasAnyClassFlags(EClassFlags::CLASS_Abstract))
			{
				Classes.Add(*It);
			}
			
		}
	}

	TMap<FString,TArray<UClass*>> UClassSortedByCategory;

	for(UClass* Class:Classes)
	{
		UClassSortedByCategory.FindOrAdd(Cast<UUTBBaseCommand>(Class->GetDefaultObject())->Category).Add(Class);
	}


	for (TTuple<FString,TArray<UClass*>> Category:UClassSortedByCategory)
	{
		FString CategoryFullName=Category.Key;
		if (CategoryFullName.IsEmpty())
		{
			CategoryFullName="Uncategorized";
		}
		TArray<FString> CategoriesSplit;
		CategoryFullName.ParseIntoArray(CategoriesSplit,TEXT("|"),1);
		TArray<TSharedPtr<FCommandTreeNode>>* CurrentArray=&CommandTree;
		TArray<TSharedPtr<FCommandTreeNode>> HierarchyHistory;
		while (!CategoriesSplit.IsEmpty())
		{
			TSharedPtr<FCommandTreeNode>* CurrentNode= CurrentArray->FindByPredicate([&CategoriesSplit](TSharedPtr<FCommandTreeNode> Node)
			{
				return CategoriesSplit[0].Compare(Node->Name)==0;
			});
			if (CurrentNode==nullptr)
			{
				TSharedPtr<FCommandTreeNode> NewNode=CurrentArray->Add_GetRef(MakeShared<FCommandTreeNode>());
				NewNode->bIsCategory=true;
				NewNode->Name=CategoriesSplit[0];
				CurrentNode=&NewNode;
				HierarchyHistory.Add(NewNode);
				CommandTreeAllNodes.Add(NewNode);
			}
			HierarchyHistory.Add(*CurrentNode);
			CurrentArray=&(*CurrentNode)->Children;
			CategoriesSplit.RemoveAt(0);
		}
		for (UClass* Class:Category.Value)
		{
			TSharedPtr<FCommandTreeNode> NewNode=CurrentArray->Add_GetRef(MakeShared<FCommandTreeNode>());
			NewNode->Name=Class->GetDisplayNameText().ToString();
			NewNode->bIsCategory=false;
			NewNode->Class=Class;
			LeafsWithParentHierarchy.Add(NewNode,HierarchyHistory);
			CommandTreeAllNodes.Add(NewNode);
		}
	}
	
	
}

void SCommandPickMenuWidget::UpdateFilteredTreeNodes(FString Text)
{

	
	TArray<TSharedPtr<FCommandTreeNode>> Leafs;
	LeafsWithParentHierarchy.GetKeys(Leafs);
	TArray<TSharedPtr<FCommandTreeNode>> FilteredLeafs;
	FilteredLeafs=Leafs.FilterByPredicate([Text](const TSharedPtr<FCommandTreeNode>& Object)
	{
		return Cast<UUTBBaseCommand>(Object->Class->GetDefaultObject())->Name.Contains(Text) || Text.IsEmpty(); 
	});

	TSet<TSharedPtr<FCommandTreeNode>> FilteredNodes;
	for (TSharedPtr<FCommandTreeNode> FilteredLeaf:FilteredLeafs)
	{
		FilteredNodes.Append(LeafsWithParentHierarchy.FindChecked(FilteredLeaf));
	}
	
	for (TSharedPtr<FCommandTreeNode> FilteredNode:CommandTreeAllNodes)
	{
		FilteredNode->FilteredChildren.Empty();
		for (TSharedPtr<FCommandTreeNode> Child:FilteredNode->Children)
		{
			if (FilteredNodes.Contains(Child) || FilteredLeafs.Contains(Child))
			{
				FilteredNode->FilteredChildren.Add(Child);
			}
		}
		TreeViewPtr->SetItemExpansion(FilteredNode,FilteredNode->FilteredChildren.Num()>0);
	}
	
	
	TreeViewPtr->RequestTreeRefresh();
}
void SCommandPickMenuWidget::OnSetExpansionRecursive(TSharedPtr<FCommandTreeNode> InTreeNode, bool bInIsItemExpanded)
{
	if (InTreeNode.IsValid() && InTreeNode->Children.Num())
	{
		TreeViewPtr->SetItemExpansion(InTreeNode, bInIsItemExpanded);

		for (TSharedPtr<FCommandTreeNode> Child : InTreeNode->Children)
		{
			OnSetExpansionRecursive(Child, bInIsItemExpanded);
		}
	}
}
