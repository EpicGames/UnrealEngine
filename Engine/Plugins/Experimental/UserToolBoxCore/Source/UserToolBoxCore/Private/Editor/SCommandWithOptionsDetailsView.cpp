// Copyright Epic Games, Inc. All Rights Reserved.


#include "Editor/SCommandWithOptionsDetailsView.h"
#include "SDropTarget.h"
#include "UserToolBoxStyle.h"
#include "UserToolBoxSubsystem.h"
#include "UTBBaseCommand.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
SCommandDetailsView::SCommandDetailsView()
{
}

SCommandDetailsView::~SCommandDetailsView()
{
}

void SCommandDetailsView::Construct(const FArguments& InArgs)
{
	Command=InArgs._Command.Get();
	OnObjectPropertyModified=InArgs._OnObjectPropertyModified;
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs Args;
	Args.bAllowSearch=true;
	Args.bShowOptions=true;
	Args.bHideSelectionTip=true;
	Args.bShowKeyablePropertiesOption=false;
	Args.bShowObjectLabel=false;
	Args.bUpdatesFromSelection=false;
	
	Args.NameAreaSettings=FDetailsViewArgs::ObjectsUseNameArea;
	CommandDetailsView=PropertyEditorModule.CreateDetailView(Args);
	CommandDetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateLambda([](const FPropertyAndParent& PropertyAndParent)
	{
		TArray<FName> PropertyNameToHide;
		PropertyNameToHide.Add(GET_MEMBER_NAME_CHECKED(UUTBBaseCommand,Category));
		return !PropertyNameToHide.Contains(PropertyAndParent.Property.GetFName());
	}));
	this->ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			CommandDetailsView.ToSharedRef()
		]
	
	];
}

void SCommandDetailsView::SetObject(UUTBBaseCommand* InCommand)
{
	
	CommandDetailsView->SetObject(InCommand);
	Command=InCommand;
}
