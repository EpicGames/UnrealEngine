// Copyright Epic Games, Inc. All Rights Reserved.


#include "Editor/SCommandWithOptionsWidget.h"

#include "IHeadMountedDisplay.h"
#include "ISequencer.h"
#include "SDropTarget.h"
#include "ToolMenuDelegates.h"
#include "ToolMenus.h"
#include "UserToolBoxStyle.h"
#include "UserToolBoxSubsystem.h"
#include "Editor/UTBTabEditor.h"
#include "UTBBaseCommand.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"

SButtonCommandWidget::SButtonCommandWidget()
{
}

SButtonCommandWidget::~SButtonCommandWidget()
{
}

void SButtonCommandWidget::Construct(const FArguments& InArgs)
{
	Command=InArgs._Command.Get();
	SButton::FArguments ButtonArgs;
	ButtonArgs._OnClicked=InArgs._OnClicked;
	ButtonArgs._Content=InArgs._Content;
	SButton::Construct(ButtonArgs);
	
}

FReply SButtonCommandWidget::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	
	return FReply::Handled().BeginDragDrop(FCommandDragDropOperation::New(Command.Get()));
}

FReply SButtonCommandWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetPressedButtons().Contains(EKeys::RightMouseButton))
	{
		UE_LOGF(LogTemp, Display, "test");
		return FReply::Handled();
	}
	return SButton::OnMouseButtonDown(MyGeometry, MouseEvent).DetectDrag(SharedThis(this),EKeys::LeftMouseButton);
}


SCommandWidget::SCommandWidget():IndexInSection(-1)
{
}

SCommandWidget::~SCommandWidget()
{
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
}

FReply SCommandWidget::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled().BeginDragDrop(FCommandDragDropOperation::New(Command.Get()));
}

FReply SCommandWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}
	return FReply::Unhandled();
}

void SCommandWidget::Construct(const FArguments& InArgs)
{
	Command=InArgs._Command.Get();
	OnTabChanged=InArgs._OnTabChanged;
	OnCommandChanged=InArgs._OnCommandChanged;
	OnCommandSelected=InArgs._OnCommandSelected;
	OnCommandDeleted=InArgs._OnCommandDeleted;
	OnDropCommand=InArgs._OnDropCommand;
	IndexInSection=InArgs._IndexInSection.Get();
	SectionName=InArgs._SectionName.Get();
	OnContextMenuOpening=InArgs._OnContextMenuOpening;
	Editor=InArgs._Editor.Get();

	OnContextMenuOpening.BindRaw(this,&SCommandWidget::CreateContextMenu);
	//if it's a bp command we should handle recompile
	if (IsValid(Command.Get()) && Command->GetClass()->ClassGeneratedBy!=nullptr)
	{
		FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &SCommandWidget::OnObjectsReplacedCallback);

	}
	RebuildWidget();
}

void SCommandWidget::OnObjectsReplacedCallback(const TMap<UObject*, UObject*>& ReplacementMap)
{
	UUTBBaseCommand* CommandPtr=Command.Get(true);
	UObject*const * ReplacementPtrPtr=ReplacementMap.Find(CommandPtr);
	
	if (ReplacementPtrPtr!=nullptr)
	{
		UUTBBaseCommand* ReplacementPtr=Cast<UUTBBaseCommand>(*ReplacementPtrPtr);
		if (IsValid(ReplacementPtr))
		{
			Command=ReplacementPtr;	
		}
	}
		
}

void SCommandWidget::RebuildWidget()
{
	if (!Command.IsValid())
	{
		return;
	}
	TSharedPtr<SWidget> IconWidget;
	if (Command->IconPath.IsEmpty())
	{
		IconWidget=SNew(SVerticalBox);
	}
	else
	{
		IconWidget=SNew(SImage)
		.Image(FUserToolBoxStyle::Get().GetBrush(FName(Command->IconPath)));
	}
	
	
	this->ChildSlot
	[
		SNew(SBox)
		.HeightOverride(125)
		.WidthOverride(250)
		.Padding(10)

		[
			SNew(SDropTarget)
			.OnAllowDrop(SDropTarget::FVerifyDrag::CreateLambda([](TSharedPtr<FDragDropOperation> Operation)
			{

				return Operation->IsOfType<FCommandDragDropOperation>();
			}))
			.OnDropped(FOnDrop::CreateLambda([this](const FGeometry& Geometry,const FDragDropEvent& Event)
			{
				if ( Event.GetOperation() && Event.GetOperation()->IsOfType<FCommandDragDropOperation>())
				{
					OnDropCommand.ExecuteIfBound(Event.GetOperationAs<FCommandDragDropOperation>()->Command.Get(),SectionName,IndexInSection);
				}
			return FReply::Handled().EndDragDrop();	
			}))
			[
				SNew(SButtonCommandWidget)
				.Command(Command.Get())
				
				.OnClicked_Lambda([this]()
				{
					
					OnCommandSelected.ExecuteIfBound(this->Command.Get());
					return FReply::Handled();
				})
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Fill)
						[
							SNullWidget::NullWidget
						]
						+SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							IconWidget.ToSharedRef()
						]
						+SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(TAttribute<FText>::Create([this]()
							{
								return FText::FromString(Command->Name);
							}))
						]
						+SVerticalBox::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Fill)
						[
							SNullWidget::NullWidget
						]
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Fill)
					.AutoWidth()
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Top)
						[
								SNew(SButton)
								.OnClicked_Lambda([this]()
								{
									bool ret=this->OnCommandDeleted.ExecuteIfBound(this->Command.Get());
									return FReply::Handled();
								})
								[
									SNew(SImage)
									.Image( FAppStyle::GetBrush("Icons.Delete") )
								]
						]
						+SVerticalBox::Slot()
						.VAlign(VAlign_Fill)
						[
							SNullWidget::NullWidget
						]
					]
				]
			]
		]
	];
}
static const FName DefaultContextBaseMenuName("UserToolboxTabEditor.DefaultContextMenuBase");
static const FName DefaultContextMenuName("UserToolboxTabEditor.DefaultContextMenu");
void SCommandWidget::RegisterContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(DefaultContextBaseMenuName))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(DefaultContextBaseMenuName);

		TWeakPtr<FUTBTabEditor> EditorPtr=Editor;
		Menu->AddDynamicSection("Copy/Move Section", FNewToolMenuDelegate::CreateLambda([EditorPtr](UToolMenu* InMenu)
		{
			UUTBBaseCommand* CurrentCommand=InMenu->Context.FindContext<UUTBBaseCommand>();
			if (!IsValid(CurrentCommand))
			{
				return;
			}
			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
			FARFilter Filter;
			Filter.bIncludeOnlyOnDiskAssets = false;
			Filter.ClassPaths = {UUserToolBoxBaseTab::StaticClass()->GetClassPathName()};
			Filter.bRecursivePaths = true;
			Filter.bRecursiveClasses = true;
			TArray<FAssetData> AssetDatas;
			AssetRegistry.GetAssets(Filter,AssetDatas);
			FToolMenuSection& Section = InMenu->AddSection("Copy/Move Section2", NSLOCTEXT("UserToolbox","CommandContextMenu", "Copy/Move"));
			Section.AddMenuEntry("Duplicate", NSLOCTEXT("UserToolbox","Duplicate Command", "Duplicate"), FText(), FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([EditorPtr,CurrentCommand]()
				{
					CurrentCommand->CopyCommand(CurrentCommand->GetOuter());
					if (EditorPtr.IsValid())
					{
						TSharedPtr<FUTBTabEditor> TabEditor=EditorPtr.Pin();
						TabEditor->NotifyTabHasChanged(Cast<UUserToolBoxBaseTab>(CurrentCommand->GetOuter()->GetOuter()));
						TabEditor->UpdateCurrentSection();
					}
					
				})));
			
			Section.AddSubMenu(
				"Copy",
				NSLOCTEXT("UserToolbox","Copy","Copy To ..."),
				NSLOCTEXT("UserToolbox","CopyTooltip","Copy a command in another user toolbox tab"),
				FNewMenuDelegate::CreateLambda([EditorPtr,CurrentCommand, AssetDatas](FMenuBuilder& MenuBuilder)
				{
					for (FAssetData AssetData:AssetDatas)
					{
						
						MenuBuilder.AddSubMenu(FText::FromName(AssetData.AssetName),FText(),
							FNewMenuDelegate::CreateLambda([EditorPtr,CurrentCommand,AssetData](FMenuBuilder& MenuBuilder)
							{
								UUserToolBoxBaseTab* CurrentTab=Cast<UUserToolBoxBaseTab>(AssetData.GetAsset());
								if (IsValid(CurrentTab))
								{
									TArray<UUTBTabSection*> Sections=CurrentTab->GetSections();
										Sections.Add(CurrentTab->GetPlaceHolderSection());
									for (UUTBTabSection* TabSection:Sections)
									{
										MenuBuilder.AddMenuEntry(FText::FromString(TabSection->SectionName), FText(), FSlateIcon(),
										FUIAction(FExecuteAction::CreateLambda([EditorPtr,CurrentCommand,TabSection,CurrentTab]()
										{
											CurrentCommand->CopyCommand(TabSection);
											if (EditorPtr.IsValid())
											{
												TSharedPtr<FUTBTabEditor> TabEditor=EditorPtr.Pin();
												TabEditor->NotifyTabHasChanged(CurrentTab);
												TabEditor->UpdateCurrentSection();
											}
											
											
										})));
									}
								}
							})
							);
					}
					
				}));
		}));
	}
}
TSharedPtr<SWidget> SCommandWidget::CreateContextMenu()
{
	RegisterContextMenu();
	UToolMenus* ToolMenus = UToolMenus::Get();
	FToolMenuContext NewContext(Command.Get());
	UToolMenu* Menu = ToolMenus->GenerateMenu(DefaultContextBaseMenuName,NewContext);
	for (const FToolMenuSection& Section : Menu->Sections)
	{
		if (Section.Blocks.Num() > 0)
		{
			return ToolMenus->GenerateWidget(Menu);
		}
	}

	return nullptr;
}
FReply SCommandWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && OnContextMenuOpening.IsBound() )
	{
		UE_LOGF(LogTemp,Display,"MouseUp");
		TSharedPtr<SWidget> ContextMenu = OnContextMenuOpening.Execute();
		if ( ContextMenu )
		{ 
			FWidgetPath WidgetPath = MouseEvent.GetEventPath() ? *MouseEvent.GetEventPath() : FWidgetPath();
			UE_LOGF(LogTemp,Display,"Pushing Menu");
			FSlateApplication::Get().PushMenu(
				AsShared(),
				WidgetPath,
				ContextMenu.ToSharedRef(),
				MouseEvent.GetScreenSpacePosition(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
				);
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}
