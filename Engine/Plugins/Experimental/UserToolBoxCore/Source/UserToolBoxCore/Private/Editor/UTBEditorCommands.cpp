// Copyright Epic Games, Inc. All Rights Reserved.


#include "Editor/UTBEditorCommands.h"
#include "Styling/AppStyle.h"
#define LOCTEXT_NAMESPACE "UTBEditorCommands"
FUTBEditorCommands::FUTBEditorCommands()
	: TCommands<FUTBEditorCommands>
(
	TEXT("UsertoolboxEditor"),
	LOCTEXT("RemoteControl", "Remote Control API"),
	NAME_None,
	FAppStyle::GetAppStyleSetName()
	
)
{
}

FUTBEditorCommands::~FUTBEditorCommands()
{
}

void FUTBEditorCommands::RegisterCommands()
{
	// Rename Entity
	UI_COMMAND(RenameSection, "Rename", "Rename the selected  section.", EUserInterfaceActionType::Button, FInputChord(EKeys::F2));


}
#undef LOCTEXT_NAMESPACE