// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
/**
 * 
 */
class FUTBEditorCommands: public TCommands<FUTBEditorCommands>
{
public:
	FUTBEditorCommands();
	~FUTBEditorCommands();

	//~ BEGIN : TCommands<> Implementation(s)

	virtual void RegisterCommands() override;

	//~ END : TCommands<> Implementation(s)


	
	TSharedPtr< FUICommandInfo > RenameSection;
};
