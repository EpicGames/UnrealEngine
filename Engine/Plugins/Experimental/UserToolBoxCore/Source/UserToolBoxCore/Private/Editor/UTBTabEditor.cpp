// Copyright Epic Games, Inc. All Rights Reserved.


#include "Editor/UTBTabEditor.h"


#include "SAssetDropTarget.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor/SCommandPickMenu.h"
#include "Editor/SCommandWithOptionsWidget.h"

#include "Styling/StyleColors.h"
#include "SDropTarget.h"
#include "ToolMenuDelegates.h"
#include "ToolMenus.h"
#include "UserToolBoxSubsystem.h"
#include "UTBEditorCommands.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Interfaces/IMainFrameModule.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Docking/SDockTab.h"
#include "PropertyEditorModule.h"
#define LOCTEXT_NAMESPACE "UserToolboxTabEditor"

const FName FUTBTabEditor::UTBTabTabId ("UTBTabEditor_Tab");
const FName FUTBTabEditor::UTBTabSectionTabId ("UTBTabEditor_Section");
const FName FUTBTabEditor::UTBCommandTabId("UTBTabEditor_Command");
const FName FUTBTabEditor::UTBCommandDetailsTabId("UTBTabEditor_CommandDetails");
const FName FUTBTabEditor::UTBTabDetailsTabId("UTBTabEditor_Details");
const FName UTBTabEditorAppIdentifier( TEXT( "UTBTabEditorApp" ) );
static const FName DefaultContextSectionBaseMenuName("UserToolboxTabEditor.SectionDefaultContextMenuBase");


class FSectionDragDrop final : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FSectionDragDrop, FDecoratedDragDropOp)

	using WidgetType = SWidget;

	FSectionDragDrop(TSharedPtr<SWidget> InWidget, const FString& SectionName)
		: SectionName(SectionName)
	{
		
	}

	/** Get the ID of the represented entity or group. */
	FString GetSectionName() const
	{
		return SectionName;
	}

	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override
	{
		FDecoratedDragDropOp::OnDrop(bDropWasHandled, MouseEvent);
	}


private:
	FString SectionName;
	TSharedPtr<SWidget> DecoratorWidget;
};


class SSectionDragHandle : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SSectionDragHandle)
	{}
	SLATE_ARGUMENT(TSharedPtr<SWidget>, Widget)
SLATE_END_ARGS()

void Construct(const FArguments& InArgs, FString _SectionName)
	{
		Widget = InArgs._Widget;
		SectionName=_SectionName;
		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(25.0f)
			.Visibility(SectionName!=UUserToolBoxBaseTab::PlaceHolderSectionName.ToString()?EVisibility::Visible:EVisibility::Collapsed)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(5.f, 0.f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
				]
			]
		];
	}

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	};

	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			TSharedPtr<FDragDropOperation> DragDropOp = CreateDragDropOperation();
			if (DragDropOp.IsValid())
			{
				return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
			}
		}

		return FReply::Unhandled();
	}

	TSharedPtr<FDragDropOperation> CreateDragDropOperation()
	{
		TSharedPtr<FSectionDragDrop> DragDropOperation = MakeShared<FSectionDragDrop>(Widget.Pin(), SectionName);
		DragDropOperation->Construct();
		return DragDropOperation;
		
	}

private:
	/** Holds the widget to display when dragging. */
	TWeakPtr<SWidget> Widget;
	/** Holds the ID of the item being dragged. */
	FString SectionName;
};
class SSectionRowWidget : public STableRow<TSharedPtr<FString>>
{
public:
	
	SLATE_BEGIN_ARGS(SSectionRowWidget)
	{}
	SLATE_ARGUMENT(FString, SectionName)
	SLATE_ARGUMENT(TSharedPtr<STableViewBase>,	Owner)
	SLATE_EVENT(FOnDrop, OnDropped)
	SLATE_EVENT(SDropTarget::FVerifyDrag, OnAllowDrop)
		/** Callback when the text is committed. */
	SLATE_EVENT( FOnTextCommitted, OnTextCommitted )
SLATE_END_ARGS()

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey()==EKeys::F2)
		{
			EnableRenaming();
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

	void Construct(const FArguments& InArgs, FString _SectionName, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		SectionName=_SectionName;
		STableRow<TSharedPtr<FString>>::Construct(STableRow<TSharedPtr<FString>>::FArguments().Content()
			[
			SNew(SBorder)
						.Padding(0.f)
						.VAlign(VAlign_Fill)
						[
							SNew(SBox)
							.HeightOverride(32.f)
							[
								SNew(SDropTarget)
								.OnDropped(InArgs._OnDropped)
								.OnAllowDrop(InArgs._OnAllowDrop)
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.VAlign(VAlign_Fill)
									.HAlign(HAlign_Center)
									.Padding(0.f, 4.f)
									.AutoWidth()
									[
										SNew(SSectionDragHandle,*SectionName)
										.Widget(SharedThis(this))
									]
									+ SHorizontalBox::Slot()
									// Group name
									.FillWidth(1.f)
									.VAlign(VAlign_Center)
									.Padding(6.f, 4.f)
									.AutoWidth()
											
									[
										SAssignNew(EditableTextBlock,SInlineEditableTextBlock)
										.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.BoldFont"))
										.Text(FText::FromString(*SectionName))
										.OnTextCommitted(InArgs._OnTextCommitted)
										.IsReadOnly(false)
									]
									

								]
							]
						]
			],InOwnerTableView);
	}

	void EnableRenaming()
	{
		if (EditableTextBlock.IsValid())
		{
			EditableTextBlock->EnterEditingMode();
		}
		
	}

private:
	/** Holds the ID of the item being dragged. */
	FString SectionName;
	TSharedPtr<SInlineEditableTextBlock> EditableTextBlock;
};

FCommandDragDropOperation::FCommandDragDropOperation():Command()
{
}

FCommandDragDropOperation::~FCommandDragDropOperation()
{
}

TSharedRef<FCommandDragDropOperation> FCommandDragDropOperation::New(UUTBBaseCommand* Command)
{
	TSharedRef<FCommandDragDropOperation> NewOp=MakeShareable(new FCommandDragDropOperation());
	NewOp->Command=Command;
	return NewOp;
}

FUTBTabEditor::FUTBTabEditor()
{
	
}

FUTBTabEditor::~FUTBTabEditor()
{
}

FLinearColor FUTBTabEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.0f, 0.0f, 0.2f, 0.5f );
}

void FUTBTabEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	IUTBTabEditor::RegisterTabSpawners(InTabManager);
	CreateOrRebuildUTBTabTabWidget();
	CreateOrRebuildUTBTabSectionTabWidget();
	UUTBTabDetailsTabWidget=CreateUTBTabDetailsTabWidget();
	UUTBCommandTabWidget=CreateUTBCommandTabWidget();
	UUTBCommandDetailsTabWidget=CreateUTBCommandDetailsTabWidget();
	
	InTabManager->RegisterTabSpawner(UTBTabTabId,FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
		.Label( LOCTEXT("UserToolBoxTab", "User Toolbox Tab") )
		.TabColorScale( GetTabColorScale() )
		[
			SNew(SBorder)
			.Padding(2)
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			[
				UUTBTabTabWidget.ToSharedRef()
			]
		];
	}))
	.SetDisplayName(LOCTEXT("UserToolBoxTab","User Toolbox Tab"));
	
	InTabManager->RegisterTabSpawner(UTBTabSectionTabId,FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
		.Label( LOCTEXT("UserToolBoxTabSection", "Tab Section") )
		.TabColorScale( GetTabColorScale() )
		[
			SNew(SBorder)
			.Padding(2)
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			[
				UUTBTabSectionTabWidget.ToSharedRef()
			]
		];
	}))
	.SetDisplayName(LOCTEXT("UserToolBoxTabSection","Tab Section"));
	
	 InTabManager->RegisterTabSpawner(UTBTabDetailsTabId,FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
	 {
	 	return SNew(SDockTab)
	 	.Label( LOCTEXT("UserToolBoxTabDetails", "Tab Details") )
	 	.TabColorScale( GetTabColorScale() )
	 	[
	 		SNew(SBorder)
	 		.Padding(2)
	 		.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
	 		[
	 			UUTBTabDetailsTabWidget.ToSharedRef()
	 		]
	 	];
	 }))
	 .SetDisplayName(LOCTEXT("UserToolBoxTabDetails","Tab Details"));
	
	InTabManager->RegisterTabSpawner(UTBCommandTabId,FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
		.Label( LOCTEXT("UserToolBoxTabCommandList", "Command List") )
		.TabColorScale( GetTabColorScale() )
		[
			SNew(SBorder)
			.Padding(2)
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			[
				UUTBCommandTabWidget.ToSharedRef()
			]
		];
	}))
	.SetDisplayName(LOCTEXT("UserToolBoxTabCommandList","Command List"));
	
	InTabManager->RegisterTabSpawner(UTBCommandDetailsTabId,FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
		.Label( LOCTEXT("UserToolBoxCommandDetails", "Command Details") )
		.TabColorScale( GetTabColorScale() )
		[
			SNew(SBorder)
			.Padding(2)
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			[
				UUTBCommandDetailsTabWidget.ToSharedRef()
			]
		];
	}))
	.SetDisplayName(LOCTEXT("UserToolBoxCommandDetails","Command Details"));
	
}

void FUTBTabEditor::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	IUTBTabEditor::UnregisterTabSpawners(InTabManager);
	InTabManager->UnregisterTabSpawner(UTBTabTabId);
	InTabManager->UnregisterTabSpawner(UTBTabSectionTabId);
	InTabManager->UnregisterTabSpawner(UTBTabDetailsTabId);
	InTabManager->UnregisterTabSpawner(UTBCommandTabId);
	InTabManager->UnregisterTabSpawner(UTBCommandDetailsTabId);
}

FName FUTBTabEditor::GetToolkitFName() const
{
	return FName("UsertoolboxTabEditor");
}

FText FUTBTabEditor::GetBaseToolkitName() const
{
	return LOCTEXT( "AppLabel", "User Toolbox Tab Editor" );
}

FString FUTBTabEditor::GetWorldCentricTabPrefix() const
{
	return "UserToolboxTab";
}

void FUTBTabEditor::PostUndo(bool bSuccess)
{
	FEditorUndoClient::PostUndo(bSuccess);
}

void FUTBTabEditor::PostRedo(bool bSuccess)
{
	FEditorUndoClient::PostRedo(bSuccess);
}

void FUTBTabEditor::InitUTBTabEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost,UUserToolBoxBaseTab* Tab)
{
	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_UserToolBoxEditor_v1" )
	->AddArea
	(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(0.33f)
			->AddTab(UTBTabSectionTabId, ETabState::OpenedTab)
		)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(0.9f)
			->AddTab(UTBTabTabId, ETabState::OpenedTab)
		)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(0.66f)
			->AddTab(UTBTabDetailsTabId, ETabState::OpenedTab)
			->AddTab(UTBCommandTabId, ETabState::OpenedTab)
			->SetForegroundTab(UTBCommandTabId)
					
		)
		->Split
		(
		FTabManager::NewStack()
			->SetSizeCoefficient(0.33f)
			
			->AddTab(UTBCommandDetailsTabId, ETabState::OpenedTab)

		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, UTBTabEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, Tab );

	RegenerateMenusAndToolbars();

	// Support undo/redo
	GEditor->RegisterForUndo(this);
	BindCommands();
}

FString FUTBTabEditor::GetCurrentSection()
{
	return CurrentSection;
}

void FUTBTabEditor::AddCommandToCurrentSection(UUTBBaseCommand* Command)
{
	UUserToolBoxBaseTab* Tab=Cast<UUserToolBoxBaseTab>(GetEditingObject());
	if (!IsValid(Tab))
	{
		return ;
	}
		
	if (!IsValid(Command))
	{
		return;
	}
	AddCommandAt(Command,CurrentSection,-1);
}

void FUTBTabEditor::RemoveCommandFromCurrentSection(UUTBBaseCommand* InCommand,bool ShouldRebuild)
{
	UUserToolBoxBaseTab* Tab=Cast<UUserToolBoxBaseTab>(GetEditingObject());
	if (!IsValid(Tab) || !IsValid(InCommand))
		return ;
	Tab->RemoveCommandFromSection(InCommand,CurrentSection);
	
	bool ret=Tab->MarkPackageDirty();
	if (ShouldRebuild)
	{
		CreateOrRebuildUTBTabTabWidget();
	}
	GEditor->GetEditorSubsystem<UUserToolboxSubsystem>()->OnTabChanged.Broadcast(Tab);
}

void FUTBTabEditor::AddCommandAt(UUTBBaseCommand* InCommand,FString Section, int Index)
{
	UUserToolBoxBaseTab* Tab=Cast<UUserToolBoxBaseTab>(GetEditingObject());
	if (!IsValid(Tab) || !IsValid(InCommand))
		return ;
	Tab->InsertCommand(InCommand,Section,Index);
	bool ret=Tab->MarkPackageDirty();
	CreateOrRebuildUTBTabTabWidget();
	
	GEditor->GetEditorSubsystem<UUserToolboxSubsystem>()->OnTabChanged.Broadcast(Tab);
}



void FUTBTabEditor::SetCurrentCommandSelection(UUTBBaseCommand* Command)
{
	CommandDetailsView->SetObject(Command);
}

void FUTBTabEditor::NotifyTabHasChanged(UUserToolBoxBaseTab* Tab)
{
	if (!IsValid(Tab))
	{
		return;
	}
	GEditor->GetEditorSubsystem<UUserToolboxSubsystem>()->OnTabChanged.Broadcast(Cast<UUserToolBoxBaseTab>(Tab));
	if (Tab==GetEditingObject())
	{
		CreateOrRebuildUTBTabTabWidget();
	}
}

void FUTBTabEditor::UpdateCommand(UUTBBaseCommand* Command)
{


	
}

void FUTBTabEditor::UpdateCurrentSection()
{
	CreateOrRebuildUTBTabTabWidget();
}

void FUTBTabEditor::CreateNewSection()
{
	UUserToolBoxBaseTab* Tab=Cast<UUserToolBoxBaseTab>(GetEditingObject());
	if (!IsValid(Tab))
	{
		return ;
	}
	int Index=0;
	FString Name="New Section";
	FString TmpName="New Section";
	
	while (Tab->GetSection(TmpName))
	{
		TmpName=Name+"_"+FString::FromInt(Index);
		Index++;
	}
	Tab->GetOrCreateSection(TmpName);
	RefreshSectionNames();
	GEditor->GetEditorSubsystem<UUserToolboxSubsystem>()->OnTabChanged.Broadcast(Tab);
}

void FUTBTabEditor::MoveSectionAfterExistingSection(FString SectionToMoveName, FString SectionToTargetName)
{
	UUserToolBoxBaseTab* Tab=Cast<UUserToolBoxBaseTab>(GetEditingObject());
	if (!IsValid(Tab))
	{
		return ;
	}
	Tab->MoveSectionAfterExistingSection(SectionToMoveName,SectionToTargetName);
	RefreshSectionNames();
	GEditor->GetEditorSubsystem<UUserToolboxSubsystem>()->OnTabChanged.Broadcast(Tab);
}

void FUTBTabEditor::RefreshSectionNames()
{
	UUserToolBoxBaseTab* Tab=Cast<UUserToolBoxBaseTab>(GetEditingObject());
	if (!IsValid(Tab))
	{
		return ;
	}
	SectionNames.Empty();
	for (UUTBTabSection* Section:Tab->GetSections())
	{
		if (!IsValid(Section))
		{
			continue;
		}
		SectionNames.Add(MakeShared<FString>(Section->SectionName));
	}
	SectionNames.Add(MakeShared<FString>(UUserToolBoxBaseTab::PlaceHolderSectionName.ToString()));
	if (SectionListView.IsValid())
	{
		SectionListView->RequestListRefresh();
	}
}

void FUTBTabEditor::RemoveCurrentSection()
{
	UUserToolBoxBaseTab* Tab=Cast<UUserToolBoxBaseTab>(GetEditingObject());
	if (!IsValid(Tab))
	{
		return ;
	}
	FString CurrentSectionName=GetCurrentSection();
	if (CurrentSectionName.IsEmpty())
	{
		return;
	}
	Tab->RemoveSection(CurrentSectionName);
	GEditor->GetEditorSubsystem<UUserToolboxSubsystem>()->OnTabChanged.Broadcast(Tab);
	RefreshSectionNames();
}

void FUTBTabEditor::EditActiveSectionName()
{
	if (SectionListView.IsValid())
	{
		for (TSharedPtr<FString> CurrentSectionName:SectionNames)
		{
			if (*CurrentSectionName==GetCurrentSection())
			{
				TSharedPtr<ITableRow> TableRow= SectionListView->WidgetFromItem(CurrentSectionName);
				if (TableRow.IsValid())
				{
					TSharedPtr<SSectionRowWidget> AsSectionRow=StaticCastSharedPtr<SSectionRowWidget>(TableRow);
					if (AsSectionRow.IsValid())
					{
						AsSectionRow->EnableRenaming();
					}
				}
			}
		}
		
	}
}

void FUTBTabEditor::RenameActiveSection(FString NewName)
{
	UUserToolBoxBaseTab* Tab=Cast<UUserToolBoxBaseTab>(GetEditingObject());
	if (!IsValid(Tab) || CurrentSection.IsEmpty() || CurrentSection==NewName || CurrentSection==UUserToolBoxBaseTab::PlaceHolderSectionName.ToString())
	{
		return ;
	}
	
	UUTBTabSection* SectionPtr=Tab->GetSection(CurrentSection);
	if (!IsValid(SectionPtr))
	{
		return;
	}
	SectionPtr->SectionName=NewName;
	Tab->MarkPackageDirty();
	CurrentSection=NewName;
	RefreshSectionNames();
	GEditor->GetEditorSubsystem<UUserToolboxSubsystem>()->OnTabChanged.Broadcast(Tab);
}

void FUTBTabEditor::BindCommands()
{
	const FUTBEditorCommands& Commands = FUTBEditorCommands::Get();

	IMainFrameModule& MainFrame = FModuleManager::Get().LoadModuleChecked<IMainFrameModule>("MainFrame");

	FUICommandList& ActionList = *MainFrame.GetMainFrameCommandBindings();

	ActionList.MapAction(
		Commands.RenameSection,
		FExecuteAction::CreateSP(this, &FUTBTabEditor::EditActiveSectionName),
		FCanExecuteAction::CreateLambda([this]()
		{
			return CurrentSection!=UUserToolBoxBaseTab::PlaceHolderSectionName.ToString();
		}));

}

void FUTBTabEditor::CreateOrRebuildUTBTabTabWidget()
{
	UUserToolBoxBaseTab* Tab=Cast<UUserToolBoxBaseTab>(GetEditingObject());
	if (!IsValid(Tab))
	{
		return ;
	}
	if (UUTBTabTabWidget.IsValid())
	{
		UUTBTabTabWidget->ClearChildren();
	}
	else
	{
		SAssignNew(UUTBTabTabWidget,SVerticalBox)	;
	}
	if (Tab->GetSection(CurrentSection)==nullptr)
	{
		UUTBTabTabWidget->AddSlot()
		[
			SNew(STextBlock)
			.Text(FText::FromString("Select a valid section"))
		];
	}
	else
	{
		UUTBTabTabWidget->AddSlot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		.HAlign(HAlign_Fill)
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(STextBlock)
				.Text(FText::FromString(CurrentSection))
			]
		];

		
		
		TSharedPtr<SWrapBox> PanelWidget= SNew(SWrapBox)
		.UseAllottedSize(true);
		TArray<TObjectPtr<UUTBBaseCommand>>& CurrentSectionArray=Tab->GetOrCreateSection(CurrentSection)->Commands;
		int Index=0;
		for (UUTBBaseCommand* Data:CurrentSectionArray)
		{
			PanelWidget->AddSlot()
			.Padding(10)
			[
				SNew(SCommandWidget)
				.Command(Data)
				.SectionName(CurrentSection)
				.IndexInSection(Index)
				.Editor(SharedThis(this))
				.OnTabChanged_Raw(this, &FUTBTabEditor::NotifyTabHasChanged)
				.OnCommandSelected_Raw(this,&FUTBTabEditor::SetCurrentCommandSelection)
				.OnCommandDeleted_Lambda([this,Tab](UUTBBaseCommand* Command)
				{
					if (IsValid(Tab))
					{
						Tab->RemoveCommand(Command);
						NotifyTabHasChanged(Tab);
						CreateOrRebuildUTBTabTabWidget();
					}
				})
			];
			Index++;
		}
		UUTBTabTabWidget->AddSlot()
		.VAlign(VAlign_Fill)
		[
			SNew(SDropTarget)
			.OnAllowDrop(SDropTarget::FVerifyDrag::CreateLambda([](TSharedPtr<FDragDropOperation> Operation)
			{

				return Operation->IsOfType<FCommandDragDropOperation>();
			}))
			.OnDropped(FOnDrop::CreateLambda([this](const FGeometry& Geometry,const FDragDropEvent& Event)
			{
				if ( Event.GetOperation() && Event.GetOperation()->IsOfType<FCommandDragDropOperation>())
				{
					if (!Event.GetOperationAs<FCommandDragDropOperation>()->Command.IsValid())
					{
						return FReply::Handled().EndDragDrop();
					}
					UUTBBaseCommand* Command=Event.GetOperationAs<FCommandDragDropOperation>()->Command.Get();
					RemoveCommandFromCurrentSection(Command);
					this->AddCommandToCurrentSection(Command);
				}
			
			return FReply::Handled().EndDragDrop();	
			}))
			[
				PanelWidget.ToSharedRef()
			]
		];
	}
	
}
void FUTBTabEditor::CreateOrRebuildUTBTabSectionTabWidget()
{
	UUserToolBoxBaseTab* Tab=Cast<UUserToolBoxBaseTab>(GetEditingObject());
	if (!IsValid(Tab))
	{
		return ;
	}
	if (UUTBTabSectionTabWidget.IsValid())
	{
		UUTBTabSectionTabWidget->ClearChildren();
	}
	else
	{
		SAssignNew(UUTBTabSectionTabWidget,SVerticalBox)	;
	}
	UUTBTabSectionTabWidget->AddSlot()
	.AutoHeight()
   [
	   SNew(SHorizontalBox)
	   +SHorizontalBox::Slot()
	   [
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(5.f, 0.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Add Section")))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ForegroundColor(FSlateColor::UseForeground())
				.ToolTipText(LOCTEXT("NewGroupToolTip", "Create new group."))
				.OnClicked_Lambda([this]()
				{
					CreateNewSection();
					return FReply::Handled();
				})
				
				[
					SNew(SBox)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
					]
				]
			]
	   ]
	   +SHorizontalBox::Slot()
	   .FillWidth(1.f)
		.VAlign(VAlign_Center)
		.Padding(2.f, 4.f)
		[
		   SNew(STextBlock)
		   .Text(FText::FromString("Sections"))
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f, 4.f, 4.f, 4.f)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Delete Selected Section")))
			.IsEnabled_Lambda([this]() { return !this->CurrentSection.IsEmpty() && CurrentSection!=UUserToolBoxBaseTab::PlaceHolderSectionName.ToString();})
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ForegroundColor(FSlateColor::UseForeground())
			.ToolTipText(LOCTEXT("DeletedSelectedSection", "Delete selected section."))
			.OnClicked_Lambda([this]()
			{
				
				RemoveCurrentSection();
				return FReply::Handled();
			})
			[
				SNew(SBox)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush("Icons.Delete"))
				]
			]
		]

   ];
	RefreshSectionNames();		
	UUTBTabSectionTabWidget->AddSlot()
	.AutoHeight()
	[		// Groups List
		SAssignNew( SectionListView,SListView<TSharedPtr<FString>>)
			.OnGenerateRow_Lambda(

			[this] (const TSharedPtr<FString> SectionName, const TSharedRef<STableViewBase>& OwnerTable)
				{
					return SNew(SSectionRowWidget,*SectionName,OwnerTable)
						.OnDropped_Lambda([this,SectionName,OwnerTable](const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
							{
								if (const TSharedPtr<FSectionDragDrop> DragDropOp = InDragDropEvent.GetOperationAs<FSectionDragDrop>())
								{
									FString DroppedSectionName=DragDropOp->GetSectionName();
									this->MoveSectionAfterExistingSection(DroppedSectionName,*SectionName);
									OwnerTable->RequestListRefresh();
									return FReply::Handled();
								}
								return FReply::Unhandled();
							})
							.OnAllowDrop_Lambda([SectionName](TSharedPtr<FDragDropOperation> DragDropOperation)
							{
								return DragDropOperation->IsOfType<FSectionDragDrop>() && !SectionName->Equals(UUserToolBoxBaseTab::PlaceHolderSectionName.ToString());
							})
						.OnTextCommitted_Lambda([this]( const FText& Text, ETextCommit::Type)
						{
							RenameActiveSection(Text.ToString());
						})

				;
				}
				
			)
			.OnSelectionChanged_Lambda([this](TSharedPtr<FString> SectionName,	ESelectInfo::Type Type)
			{
				if (SectionName.IsValid())
				{
					SetCurrentSection(*SectionName);
				}
				else
				{
					SetCurrentSection("");
				}
			})
			.SelectionMode(ESelectionMode::Single)
			.ListItemsSource(&SectionNames)
			.OnContextMenuOpening_Lambda([this]()
			{
				IMainFrameModule& MainFrame = FModuleManager::Get().LoadModuleChecked<IMainFrameModule>("MainFrame");
					FMenuBuilder MenuBuilder(true, MainFrame.GetMainFrameCommandBindings());
					MenuBuilder.BeginSection("Common");
					MenuBuilder.AddMenuEntry(FUTBEditorCommands::Get().RenameSection, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCommands.Rename")));
					MenuBuilder.EndSection();
					return MenuBuilder.MakeWidget();
			})
			.ClearSelectionOnClick(true)
	];
}
 TSharedPtr<SWidget> FUTBTabEditor::CreateUTBTabDetailsTabWidget()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs Args;
	Args.bAllowSearch=true;
	Args.bShowOptions=true;
	Args.bHideSelectionTip=true;
	Args.bShowKeyablePropertiesOption=false;
	Args.bShowObjectLabel=false;

	Args.bUpdatesFromSelection=false;
	Args.NameAreaSettings=FDetailsViewArgs::ObjectsUseNameArea;
	TSharedRef<IDetailsView> DetailView=PropertyEditorModule.CreateDetailView(Args);
	DetailView->SetObject(Cast<UUserToolBoxBaseTab>(GetEditingObject()));
	return 	DetailView;
}

 TSharedPtr<SWidget> FUTBTabEditor::CreateUTBCommandTabWidget()
{
	return SNew(SCommandPickMenuWidget);
}

TSharedPtr<SWidget> FUTBTabEditor::CreateUTBCommandDetailsTabWidget()
{
	return SAssignNew(CommandDetailsView,SCommandDetailsView)
	.OnObjectPropertyModified_Raw(this,&FUTBTabEditor::UpdateCommand);
}

void FUTBTabEditor::SetCurrentSection(FString SectionName)
{
	CurrentSection=SectionName;
	CreateOrRebuildUTBTabTabWidget();
	OnCurrentSectionChange.Broadcast(CurrentSection);
}

#undef LOCTEXT_NAMESPACE 
