// Copyright Epic Games, Inc. All Rights Reserved.


#include "SVerticalTextBlock.h"

SVerticalTextBlock::SVerticalTextBlock()
{
}

SVerticalTextBlock::~SVerticalTextBlock()
{
}

void SVerticalTextBlock::Construct(const FArguments& InArgs)
{
	STextBlock::Construct(STextBlock::FArguments()
		.Font(InArgs._Font)
		.Justification(InArgs._Justification)
		.Margin(InArgs._Margin)
		.Text(InArgs._Text)
		.HighlightColor(InArgs._HighlightColor)
		.HighlightShape(InArgs._HighlightShape)
		.HighlightText(InArgs._HighlightText)
		.OverflowPolicy(InArgs._OverflowPolicy)
		.ShadowOffset(InArgs._ShadowOffset)
		.StrikeBrush(InArgs._StrikeBrush)
		.TextStyle(InArgs._TextStyle)
		.TransformPolicy(InArgs._TransformPolicy)
		.WrappingPolicy(InArgs._WrappingPolicy)
		.AutoWrapText(InArgs._AutoWrapText)
		.ColorAndOpacity(InArgs._ColorAndOpacity)
		.WrapTextAt(InArgs._WrapTextAt)
		.LineBreakPolicy(InArgs._LineBreakPolicy)
		.LineHeightPercentage(InArgs._LineHeightPercentage)
		.Justification(InArgs._Justification)
		.MinDesiredWidth(InArgs._MinDesiredWidth)
		.TextShapingMethod(InArgs._TextShapingMethod)
		.TextFlowDirection(InArgs._TextFlowDirection)
		.SimpleTextMode(InArgs._SimpleTextMode)
		);

	SetRenderTransformPivot(FVector2d(0,0));
	const FSlateRenderTransform Rotate90(FQuat2D(FMath::DegreesToRadians(90.f)));
	SetRenderTransform(Rotate90);
	
}

FVector2D SVerticalTextBlock::ComputeDesiredSize(const float X) const
{
	FVector2D WantedSize=STextBlock::ComputeDesiredSize(X);
	double Tmp=WantedSize.X;
	WantedSize.X=WantedSize.Y;
	WantedSize.Y=Tmp;
	return WantedSize;
}
