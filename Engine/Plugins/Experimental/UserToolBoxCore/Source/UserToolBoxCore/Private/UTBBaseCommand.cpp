// Copyright Epic Games, Inc. All Rights Reserved.

#include "UTBBaseCommand.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Dialog/SCustomDialog.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "UTBBaseTab.h"
#include "Widgets/Layout/SBox.h"
#include "ScopedTransaction.h"
#include "Modules/ModuleManager.h"

void UUTBBaseCommand::Execute()
{
}

void UUTBBaseCommand::ExecuteCommand()
{
	
	if (bShowParameters)
	{
		if (!DisplayParameters())
		{
			return;
		}
			
	}
	if (bIsTransaction)
	{
		const FScopedTransaction Transaction =FScopedTransaction(nullptr,FText::FromString(Name),nullptr);
		Execute();	
	}
	else
	{
		Execute();
	}
	
}

bool UUTBBaseCommand::DisplayParameters()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs Args;
	Args.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	Args.bHideSelectionTip = true;
	Args.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	Args.bAllowSearch = false;

	
	TSharedRef<IDetailsView> DetailsView=PropertyEditorModule.CreateDetailView(Args);
	
	DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateLambda([](const FPropertyAndParent& PropertyAndParent)
	{
		FProperty * Property=const_cast<FProperty*>(&PropertyAndParent.Property);
		return !StaticClass()->HasProperty(Property);

	}));
	DetailsView->SetObject(this);
	TSharedRef<SCustomDialog> OptionsWindow =
	SNew(SCustomDialog)
	.Title(FText::FromString(Name+" properties"))
	.Content()
	[
		SNew(SBox)
		.MinDesiredWidth(400)
		[
			DetailsView

		]
		
	]
	.ContentAreaPadding(FMargin(0.f, 0.f, 0.f, 0.f))
	.HAlignContent(EHorizontalAlignment::HAlign_Left)
	.VAlignContent(EVerticalAlignment::VAlign_Top)
	.Buttons
	(
		{
		SCustomDialog::FButton(FText::FromString("Ok")),
		SCustomDialog::FButton(FText::FromString("Cancel")),
		}
	);
	return OptionsWindow->ShowModal()==0;
	
}

UUTBBaseCommand* UUTBBaseCommand::CopyCommand( UObject* Owner)const 
{
	UUTBTabSection* NewSection=Cast<UUTBTabSection>(Owner);
	if (IsValid(NewSection))
	{
		UUTBBaseCommand* NewCommand=DuplicateObject(this,Owner);
		NewSection->Commands.Add(NewCommand);
		return NewCommand;	
	}
	return nullptr;
}

void UUTBBaseCommand::AddObjectToTransaction(UObject* Object)
{
	if (IsValid(Object))
	{
		Object->Modify();
	}
	
}

void UUTBBaseCommand::AddObjectsToTransaction(TArray<UObject*> Objects)
{
	for (UObject* Object:Objects)
	{
		if (IsValid(Object))
		{
			Object->Modify();
		}
	}
}


