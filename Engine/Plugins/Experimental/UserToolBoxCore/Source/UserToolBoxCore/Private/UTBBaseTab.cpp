// Copyright Epic Games, Inc. All Rights Reserved.


#include "UTBBaseTab.h"

#include "EditorUtilityBlueprint.h"
#include "UICommandsScriptingSubsystem.h"
#include "UserToolBoxBaseBlueprint.h"
#include "UserToolBoxSubsystem.h"
#include "UTBBaseUICommand.h"
#include "UTBBaseUITab.h"
#include "Engine/Engine.h"
#include "Editor.h"

const FName UUserToolBoxBaseTab::PlaceHolderSectionName="Hidden";


TSharedPtr<SWidget> UUserToolBoxBaseTab::GenerateUI()
{
	return GenerateUI(nullptr,FUITemplateParameters());
}
TSharedPtr<SWidget>  UUserToolBoxBaseTab::GenerateUI(const TSubclassOf<UUTBDefaultUITemplate> TabUiClass,const FUITemplateParameters& Params)
{
	RemoveInvalidCommand();
	TSubclassOf<UUTBDefaultUITemplate> RealTabUiClass=TabUiClass;
	if (RealTabUiClass==nullptr)
	{
		RealTabUiClass= this->TabUI;
	}
	if (RealTabUiClass==nullptr)
	{
		RealTabUiClass=UUTBDefaultUITemplate::StaticClass();
	}
	if (IsValid(RealTabUiClass))
	{
		UUTBDefaultUITemplate* CurTabUI;
		CurTabUI= NewObject<UUTBDefaultUITemplate>(GetTransientPackage(),RealTabUiClass);
		TSharedPtr<SWidget> Widget=CurTabUI->BuildTabUI(this,Params);
		return Widget;
	}
	return TSharedPtr<SWidget>();
}
void UUserToolBoxBaseTab::PostLoad()
{
	Super::PostLoad();
	OnObjectReplacedHandle=FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this,&UUserToolBoxBaseTab::ReplaceCommands);
}

void UUserToolBoxBaseTab::BeginDestroy()
{
	UnregisterCommand();
	if (OnObjectReplacedHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectsReplaced.Remove(OnObjectReplacedHandle);	
	}
	Super::BeginDestroy();
}

void UUserToolBoxBaseTab::InsertCommandFromClass(const TSubclassOf<UUTBBaseCommand> InClass,const FString Section,const int Index)
{
	if (InClass==nullptr)
	{
		return;
	}
	InsertCommand(Cast<UUTBBaseCommand>(InClass->GetDefaultObject()),Section,Index);
}

void UUserToolBoxBaseTab::InsertCommand(const UUTBBaseCommand* InCommand,const FString SectionName,const int Index)
{
	UUTBTabSection* CurrentSection=GetOrCreateSection(SectionName);
	UUTBBaseCommand* NewCommand=DuplicateObject(InCommand,CurrentSection);
	if (Index<0 || Index>=CurrentSection->Commands.Num())
	{
		CurrentSection->Commands.Add(NewCommand);
	}
	else
	{
		CurrentSection->Commands.Insert(NewCommand,Index);
	}
}


void UUserToolBoxBaseTab::RemoveCommand(const FString SectionName,const int Index)
{
	UUTBTabSection* CurrentSection=GetSection(SectionName);
	if (CurrentSection!=nullptr)
	{
		CurrentSection->Commands.RemoveAt(Index);
	}
}



UUTBTabSection* UUserToolBoxBaseTab::GetSection(const FString SectionName)
{
	Sections.Remove(nullptr);
	if (SectionName==PlaceHolderSectionName.ToString())
	{
		return PlaceholderSection;
	}
	for (UUTBTabSection* Section:Sections)
	{
		if (Section->SectionName==SectionName)
		{
			return Section;
		}
	}
	return nullptr;
}

UUTBTabSection* UUserToolBoxBaseTab::GetOrCreateSection( const FString SectionName)
{
	if (SectionName==PlaceHolderSectionName.ToString())
	{
		return GetPlaceHolderSection();
	}
	UUTBTabSection* CurrentSection=GetSection(SectionName);
	if (IsValid(CurrentSection))
	{
		return CurrentSection;
	}
	CurrentSection=NewObject<UUTBTabSection>(this);
	CurrentSection->SectionName=SectionName;
	Sections.Add(CurrentSection);
	return CurrentSection;
}

void UUserToolBoxBaseTab::RemoveSection( UUTBTabSection* Section)
{
	if (Section==PlaceholderSection)
	{
		return;
	}
	Sections.Remove(Section);
}

void UUserToolBoxBaseTab::RemoveSection(const FString SectionName)
{
	UUTBTabSection* Section=GetSection(SectionName);
	if (IsValid(Section))
	{
		RemoveSection(Section);
	}
}

void UUserToolBoxBaseTab::RemoveCommand(UUTBBaseCommand* Command)
{
	for (UUTBTabSection* Section:Sections)
	{
		Section->Commands.Remove(Command);
	}
	if (IsValid(PlaceholderSection))
	{
		PlaceholderSection->Commands.Remove(Command);
	}
}

void UUserToolBoxBaseTab::RemoveCommandFromSection(UUTBBaseCommand* Command,const FString SectionName)
{
	UUTBTabSection* Section=GetSection(SectionName);
	if (IsValid(Section))
	{
		Section->Commands.Remove(Command);
	}
}

void UUserToolBoxBaseTab::ReplaceCommands(const TMap<UObject*, UObject*>& ReplacementMap)
{
	Sections.Remove(nullptr);
	for (UUTBTabSection* Section:Sections)
	{
		for (UUTBBaseCommand* Command:Section->Commands)
		{
			UObject*const* Result=ReplacementMap.Find(Command);
			if (Result!=nullptr)
			{
				Section->Commands[Section->Commands.Find(Command)]=Cast<UUTBBaseCommand>(*Result);	
			}
		}
	}
	if (IsValid(PlaceholderSection))
	{
		for (UUTBBaseCommand* Command:PlaceholderSection->Commands)
		{
			UObject*const* Result=ReplacementMap.Find(Command);
			if (Result!=nullptr)
			{
				PlaceholderSection->Commands[PlaceholderSection->Commands.Find(Command)]=Cast<UUTBBaseCommand>(*Result);	
			}
		}	
	}
}

TArray<UUTBTabSection*> UUserToolBoxBaseTab::GetSections()
{
	return Sections;
}

void UUserToolBoxBaseTab::MoveSectionAfterExistingSection(const FString SectionToMoveName,const FString SectionToTargetName)
{
	
	UUTBTabSection* SectionToMove=GetSection(SectionToMoveName);
	UUTBTabSection* SectionToTarget=GetSection(SectionToTargetName);
	if (!(IsValid(SectionToMove)&&IsValid(SectionToTarget))&&SectionToMove==SectionToTarget)
	{
		return;
	}
	int32 Index=Sections.Find(SectionToTarget);
	Sections.Remove(SectionToMove);
	Sections.Insert(SectionToMove,Index);
}

UUTBTabSection* UUserToolBoxBaseTab::GetPlaceHolderSection()
{
	if (!IsValid(PlaceholderSection))
	{
		PlaceholderSection=NewObject<UUTBTabSection>(this);
		PlaceholderSection->SectionName=PlaceHolderSectionName.ToString();
	}
	return PlaceholderSection;
}

bool UUserToolBoxBaseTab::ContainsCommand(const UUTBBaseCommand* Command)const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!IsThisNotNull(this, "UUserToolBoxBaseTab::ContainsCommand") || !IsValidChecked(this))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return false;
	}
	for (UUTBTabSection* Section:Sections)
	{
		if (Section->Commands.Contains(Command))
		{
			return true;
		}
	}
	return false;
}

void UUserToolBoxBaseTab::RegisterCommand()
{
	UnregisterCommand();
	UUICommandsScriptingSubsystem* UICommandsScriptingSubsystem = GEngine->GetEngineSubsystem<UUICommandsScriptingSubsystem>();
	if (!IsValid(UICommandsScriptingSubsystem))
	{
		return;
	}
	for (UUTBTabSection* Section:Sections)
	{
		FString SectionName=Section->SectionName;
		FName NewSetName=FName(this->GetName());
		if (!UICommandsScriptingSubsystem->IsCommandSetRegistered(NewSetName))
		{
			UICommandsScriptingSubsystem->RegisterCommandSet(NewSetName);	
		}
		
		
		for (UUTBBaseCommand* Command:Section->Commands)
		{
			if (!IsValid(Command))
			{
				continue;
			}
			if (Command->KeyboardShortcut.IsValidChord())
			{
				FScriptingCommandInfo ScriptingCommandInfo;
				ScriptingCommandInfo.InputChord=Command->KeyboardShortcut;
				ScriptingCommandInfo.Description=FText::FromString(Command->Tooltip);
				ScriptingCommandInfo.Label=FText::FromString(Section->SectionName+Command->Name);
				ScriptingCommandInfo.Set=NewSetName;
				ScriptingCommandInfo.ContextName="LevelViewport";
				ScriptingCommandInfo.Name=FName(Command->Name);
				FExecuteCommand ExecuteCommand;
				ExecuteCommand.BindUFunction(Command,FName("ExecuteCommand"));
				UICommandsScriptingSubsystem->RegisterCommand(ScriptingCommandInfo,ExecuteCommand,true);
				ScriptingCommandInfo.ContextName="ContentBrowser";
				UICommandsScriptingSubsystem->RegisterCommand(ScriptingCommandInfo,ExecuteCommand,true);
			}
		}
	}
}

void UUserToolBoxBaseTab::UnregisterCommand()
{
	UUICommandsScriptingSubsystem* UICommandsScriptingSubsystem = GEngine->GetEngineSubsystem<UUICommandsScriptingSubsystem>();
	if (!IsValid(UICommandsScriptingSubsystem))
	{
		return;
	}
	if (UICommandsScriptingSubsystem->IsCommandSetRegistered(FName(GetName())))
	{
		UICommandsScriptingSubsystem->UnregisterCommandSet(FName(GetName()));	
	}
}

void UUserToolBoxBaseTab::RemoveInvalidCommand()
{
	
	for (UUTBTabSection* Section:Sections)
	{
		if (IsValid(Section))
		{
			Section->Commands.RemoveAll([](UUTBBaseCommand* Command)
			{
				return !IsValid(Command);
			});
		}
	}
}


SUserToolBoxTabWidget::~SUserToolBoxTabWidget()
{
	if (OnTabChangedDelegate.IsValid())
	{
		UUserToolboxSubsystem* UTBSubsystem=GEditor->GetEditorSubsystem<UUserToolboxSubsystem>();
		if (IsValid(UTBSubsystem))
		{
			UTBSubsystem->OnTabChanged.Remove(OnTabChangedDelegate);
			OnTabChangedDelegate.Reset();	
		}
		
		FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnObjectPropertyChangedDelegate);
	}
	if (IsValid(Tab))
	{
		Tab->UnregisterCommand();
	}
}

void SUserToolBoxTabWidget::Construct(const FArguments& InArgs)
{
	Tab=InArgs._Tab.Get();
	UIOverride=InArgs._UIOverride.Get();
	Parameters=InArgs._UIParameters.Get();
	UpdateTab(Tab);
	if (!OnTabChangedDelegate.IsValid())
	{
		UUserToolboxSubsystem* UTBSubsystem=GEditor->GetEditorSubsystem<UUserToolboxSubsystem>();
		OnTabChangedDelegate=UTBSubsystem->OnTabChanged.AddSP(this,&SUserToolBoxTabWidget::UpdateTab);
		OnObjectPropertyChangedDelegate=FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this,&SUserToolBoxTabWidget::OnObjectModified);
	}
	Tab->RegisterCommand();
}

void SUserToolBoxTabWidget::UpdateTab(UUserToolBoxBaseTab* InTab)
{
	if (!IsValid(Tab) || Tab!=InTab )
		return;
	if (!IsValid(Tab))
	{
		ChildSlot
		[
			SNullWidget::NullWidget
		];
		return;
	}
	ChildSlot
	[
		Tab->GenerateUI(IsValid(UIOverride)?UIOverride:nullptr,Parameters).ToSharedRef()
	];	
	Tab->RegisterCommand();
}

void SUserToolBoxTabWidget::OnObjectModified(UObject* Object, FPropertyChangedEvent& PropertyChangedEventEvent)
{
	/*
	 * We are looking for each modified object to check update of a command and update the UI
	 */
	if (Object->IsA<UUTBBaseCommand>())
	{
		if (!IsValid(Tab))
		{
			return;
		}
		if (Tab->ContainsCommand(Cast<UUTBBaseCommand>(Object)))
		{
			UpdateTab(Tab);
		}
	}
	if (Object->IsA<UUserToolBoxBaseTab>())
	{
		if (!IsValid(Tab))
		{
			return;
		}
		if (Tab==Object)
		{
			if (PropertyChangedEventEvent.Property->GetName()=="TabUI")
			{
				UpdateTab(Tab);
			}
		}
	}
}
