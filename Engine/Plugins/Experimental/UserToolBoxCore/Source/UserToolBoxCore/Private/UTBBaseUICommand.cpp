// Copyright Epic Games, Inc. All Rights Reserved.


#include "UTBBaseUICommand.h"
#include "UTBBaseCommand.h"
#include "UserToolBoxStyle.h"
#include "ClassViewerFilter.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Components/CanvasPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"

bool UBaseCommandNativeUI::IsSupportingCommandClass(TSubclassOf<UUTBBaseCommand> CommandClass)
{

	return true;
}

void UBaseCommandNativeUI::SetCommand(UUTBBaseCommand* Command)
{
	MyCommand=Command;
	if (MyCommand->GetClass()->ClassGeneratedBy!=nullptr)
	{
		FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this,&UBaseCommandNativeUI::ReplaceCommand);
	}

}

TSharedRef<SWidget> UBaseCommandNativeUI::GetUI()
{

	TSharedPtr<SWidget> Widget;
	TSharedPtr<SVerticalBox> VerticalBox;
	
	SAssignNew(Widget,SVerticalBox)
	+SVerticalBox::Slot()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[

			SNew(SButton)
			.ButtonColorAndOpacity(FLinearColor::Gray)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
				
			.OnClicked_Lambda([this]
				()
			{
				check(MyCommand)
				
				MyCommand->ExecuteCommand();
				return FReply::Handled();
			})
			[

				SAssignNew(VerticalBox, SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.AutoWrapText(true)
					.ColorAndOpacity(FLinearColor::White)
					.Text(FText::FromString(MyCommand->Name))
				]
			]
	
		
	];
	
	if (!MyCommand->IconPath.IsEmpty())
	{
		VerticalBox->InsertSlot(0)
			.AutoHeight()
			.HAlign(HAlign_Center)
			.Padding(1)
			[
				SNew(SImage)
				.Image(FUserToolBoxStyle::Get().GetBrush(FName(MyCommand->IconPath)))
			];
	}
	return Widget.ToSharedRef();
	
	
}

void UBaseCommandNativeUI::ReplaceCommand(const TMap<UObject*, UObject*>& Map)
{
	if (Map.Find(MyCommand)!=nullptr)
	{
		MyCommand=Cast<UUTBBaseCommand>(*Map.Find(MyCommand));
	}
		
}



void UBaseCommandNativeUI::BeginDestroy()
{
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	UObject::BeginDestroy();
}

// Add default functionality here for any IUTBBaseUICommand functions that are not pure virtual.
void UBaseCommandNativeUI::ExecuteCurrentCommand()
{
	if (MyCommand!=nullptr)
	{
		MyCommand->ExecuteCommand();
	}
}






void UUTBCommandUMGUI::ExecuteCommand()
{
	ExecuteCurrentCommand();
}

bool UUTBCommandUMGUI::IsSupportingCommandClass(TSubclassOf<UUTBBaseCommand> CommandClass)
{
	return DoesSupportClass(CommandClass);
}

void UUTBCommandUMGUI::SetCommand(UUTBBaseCommand* _Command)
{
	Command=_Command;
	if (Command->GetClass()->ClassGeneratedBy!=nullptr)
	{
		FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this,&UUTBCommandUMGUI::ReplaceCommand);
	}
}

void UUTBCommandUMGUI::ExecuteCurrentCommand()
{
	if (Command!=nullptr)
	{
		Command->ExecuteCommand();
	}
}

TSharedRef<SWidget> UUTBCommandUMGUI::GetUI()
{
	return TakeWidget();
}

void UUTBCommandUMGUI::ReplaceCommand(const TMap<UObject*, UObject*>& Map)
{
	if (Map.Find(Command)!=nullptr)
	{
		Command=Cast<UUTBBaseCommand>(*Map.Find(Command));
	}
		
}

void UUTBCommandUMGUI::BeginDestroy()
{
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);

	Super::BeginDestroy();
}

