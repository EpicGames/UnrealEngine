// Copyright Epic Games, Inc. All Rights Reserved.


#include "UTBBaseUITab.h"
#include "Widgets/Layout/SScrollBox.h"
#include "UserToolBoxStyle.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Dialog/SCustomDialog.h"
#include "UTBBaseTab.h"
#include "UTBBaseCommand.h"
#include "UTBBaseUICommand.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Framework/Commands/InputBindingManager.h"

TSharedPtr<SWidget> UUTBDefaultUITemplate::BuildTabUI(UUserToolBoxBaseTab* Tab, const FUITemplateParameters& Params)
{
	if (Tab==nullptr)
	{
		return TSharedPtr<SWidget>();
	}
	TSharedPtr<SScrollBox> ScrollBox;
	SAssignNew(ScrollBox,SScrollBox)
	.Orientation(Orient_Vertical);
	if (!Params.bForceHideSettingsButton)
	{
		ScrollBox->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.Visibility_Lambda([Tab]()
			{
				return Tab->IsSettingShouldBeVisible?EVisibility::Visible:EVisibility::Collapsed;
			})
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.Adjust"))
			]
			.OnClicked_Lambda([Tab]()
			{
			
				UAssetEditorSubsystem* AssetEditorSubsystem=GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
				if (IsValid(AssetEditorSubsystem) )
				{
					AssetEditorSubsystem->OpenEditorForAsset(Tab);
				}

				return FReply::Handled();
			})
		];	
	}
	
	
	for (UUTBTabSection* Section:Tab->GetSections())
	{
		if (!IsValid(Section))
		{
			continue;
		}
		TSharedPtr<SWrapBox> WrapBox;
		FString SectionName=Section->SectionName;
		if (Params.bPostfixSectionWithTabName)
		{
			SectionName+= " - "+Tab->Name;
		}
		if (Params.bPrefixSectionWithTabName)
		{
			SectionName= Tab->Name+" - "+SectionName;
		}
		
		ScrollBox->AddSlot()
		[
			SNew(SExpandableArea)
			.AreaTitle(FText::FromString(SectionName))
			.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
			.BodyBorderBackgroundColor(FLinearColor::Transparent)
			.BodyContent()
			[
				SAssignNew(WrapBox,SWrapBox)
				.UseAllottedSize(true)
				
			]
		];
		TMap<int,TSet<int>> SlotToSkip;
		for (UUTBBaseCommand* Command:Section->Commands)
		{
			if (!IsValid(Command))
			{
				continue;
			}
			UClass * UIToUse=UBaseCommandNativeUI::StaticClass();
			// checking the override
			// least priority first
			if (Tab->DefaultCommandUIOverride!=nullptr)
			{
				IUTBUICommand* UIInterface=Cast<IUTBUICommand>(Tab->DefaultCommandUIOverride->GetDefaultObject());
				if (UIInterface!=nullptr)
				{
					if (UIInterface->IsSupportingCommandClass(Command->GetClass()))
					{
						UIToUse=Tab->DefaultCommandUIOverride;
					}
				}
			}
			if (Command->UIOverride!=nullptr)
			{
				IUTBUICommand* UIInterface=Cast<IUTBUICommand>(Command->UIOverride->GetDefaultObject());
				if (UIInterface!=nullptr)
				{
					if (UIInterface->IsSupportingCommandClass(Command->GetClass()))
					{
						UIToUse=Command->UIOverride;
					}
				}
			}
			Command->UI=NewObject<UObject>(GetTransientPackage(),UIToUse);
			Cast<IUTBUICommand>(Command->UI)->SetCommand(Command);
			WrapBox->AddSlot()
			.FillEmptySpace(false)
			[
				SNew(SBox)
				.WidthOverride(120)
				.HeightOverride(70)
				[
				Cast<IUTBUICommand>(Command->UI)->GetUI()
				]	
			];
		}
	}
	return ScrollBox;
}


TSharedPtr<SWidget> UUTBToolBarTabUI::BuildTabUI(UUserToolBoxBaseTab* Tab, const FUITemplateParameters& Params)
{
	TSharedRef<SVerticalBox> ToolBoxVBox = SNew(SVerticalBox);
	if (!Params.bForceHideSettingsButton)
	{
		ToolBoxVBox->AddSlot()
		.VAlign(EVerticalAlignment::VAlign_Top)
		.HAlign(EHorizontalAlignment::HAlign_Right)
		.AutoHeight()
		.Padding(0)
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.Visibility_Lambda([Tab]()
			{
				return Tab->IsSettingShouldBeVisible?EVisibility::Visible:EVisibility::Collapsed;
			})
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.Adjust"))
			]
			.OnClicked_Lambda([Tab]()
			{
				UAssetEditorSubsystem* AssetEditorSubsystem=GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
				if (IsValid(AssetEditorSubsystem) )
				{
					AssetEditorSubsystem->OpenEditorForAsset(Tab);
				}
				return FReply::Handled();
			})
		];	
	}
	
	TArray<UUTBTabSection*> Sections=Tab->GetSections();
	TMap<FString,TArray<TSharedPtr<FUICommandInfo>>> UICommandInfosPerSection;
	TSharedPtr<class FUICommandList> TabCommands = MakeShareable(new FUICommandList);
	for (UUTBTabSection* Section:Sections)
	{
		TSharedPtr<FBindingContext>   BindingContext= MakeShared<FBindingContext>(FName(FString("UTB.")+Tab->GetName()+"."+Section->GetName()),FText::FromString(FString("UTB.")+Tab->GetName()+"."+Section->GetName()),NAME_None,FAppStyle::Get().GetStyleSetName());
		TSharedPtr<FToolBarBuilder> ToolbarBuilder=GetBuilder(TabCommands, FMultiBoxCustomization(FName(Tab->Name)));
		ToolbarBuilder->SetStyle(&FAppStyle::Get(), "PaletteToolBar");
		FSlateIcon EmptyIcon(FAppStyle::Get().GetStyleSetName(),"TableView.Header.Column");
		int CommandCounter=0;
		for (UUTBBaseCommand* Command:Section->Commands)
		{
			FString CommandName=Command->Name+"_"+FString::FromInt(CommandCounter);
			TSharedPtr<FUICommandInfo> CommandInfo=FInputBindingManager::Get().FindCommandInContext(BindingContext->GetContextName(),FName(CommandName));
			if (CommandInfo.IsValid())
			{
				FInputBindingManager::Get().RemoveInputCommand(BindingContext.ToSharedRef(),CommandInfo.ToSharedRef());
			}
			FUICommandInfo::MakeCommandInfo(
				BindingContext.ToSharedRef(),
				CommandInfo,FName(CommandName),
				FText::FromString(Command->Name),
				FText::FromString(Command->Tooltip),
				Command->IconPath.IsEmpty()?EmptyIcon:
				FSlateIcon(FUserToolBoxStyle::GetStyleSetName(), FName(Command->IconPath)),
				EUserInterfaceActionType::Button,FInputChord(),FInputChord(),NAME_None
				);	
			TabCommands->MapAction(CommandInfo,FExecuteAction::CreateUObject(Command,&UUTBBaseCommand::ExecuteCommand));
			UClass * UIToUse=UBaseCommandNativeUI::StaticClass();
			if (Tab->DefaultCommandUIOverride!=nullptr)
			{
				IUTBUICommand* UIInterface=Cast<IUTBUICommand>(Tab->DefaultCommandUIOverride->GetDefaultObject());
				if (UIInterface!=nullptr)
				{
					if (UIInterface->IsSupportingCommandClass(Command->GetClass()))
					{
						UIToUse=Tab->DefaultCommandUIOverride;
					}
				}
			}
			if (Command->UIOverride!=nullptr)
			{
				IUTBUICommand* UIInterface=Cast<IUTBUICommand>(Command->UIOverride->GetDefaultObject());
				if (UIInterface!=nullptr)
				{
					if (UIInterface->IsSupportingCommandClass(Command->GetClass()))
					{
						UIToUse=Command->UIOverride;
					}
				}
			}
			Command->UI=NewObject<UObject>(GetTransientPackage(),UIToUse);
			Cast<IUTBUICommand>(Command->UI)->SetCommand(Command);
			if (UIToUse!=UBaseCommandNativeUI::StaticClass())
			{
				ToolbarBuilder->AddToolBarWidget(Cast<IUTBUICommand>(Command->UI)->GetUI());	
			}
			else
			{
				ToolbarBuilder->AddToolBarButton(CommandInfo);
			}
			ToolbarBuilder->AddSeparator();
			CommandCounter++;
		}
		FString SectionName=Section->SectionName;
		if (Params.bPostfixSectionWithTabName)
		{
			SectionName+= " - "+Tab->Name;
		}
		if (Params.bPrefixSectionWithTabName)
		{
			SectionName= Tab->Name+" - "+SectionName;
		}
		
		ToolBoxVBox->AddSlot()
		.AutoHeight()
		.VAlign(EVerticalAlignment::VAlign_Top)
		.Padding(FMargin(0.0, 0.0))
		[
			SNew(SExpandableArea)
			.AreaTitle(FText::FromString(SectionName))
			.AreaTitleFont(FAppStyle::Get().GetFontStyle("NormalFont"))
			.BorderImage(FAppStyle::Get().GetBrush("PaletteToolbar.ExpandableAreaHeader"))
			.BodyBorderImage(FAppStyle::Get().GetBrush("PaletteToolbar.ExpandableAreaBody"))
			.HeaderPadding(FMargin(4.f))
			.InitiallyCollapsed(true)
			.Padding(FMargin(4.0, 0.0))
			.BodyContent()
			[
				SNew(SBox)
				[
					ToolbarBuilder->MakeWidget()
				]	
			]
		];
	}
	ToolBoxVBox->AddSlot()
	.AutoHeight()
	.VAlign(EVerticalAlignment::VAlign_Fill)
	[
		SNew(SVerticalBox)
	];
	return ToolBoxVBox;
}


TSharedPtr<FToolBarBuilder> UUTBToolBarTabUI::GetBuilder(TSharedPtr<const FUICommandList> InCommandList,
	FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender, const bool InForceSmallIcons)
{
	return MakeShareable(new FToolBarBuilder(InCommandList,InCustomization,InExtender,InForceSmallIcons));
}

TSharedPtr<FToolBarBuilder> UUTBPaletteTabUI::GetBuilder(TSharedPtr<const FUICommandList> InCommandList,
	FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender, const bool InForceSmallIcons)
{
	return MakeShareable(new FUniformToolBarBuilder(InCommandList,InCustomization,InExtender,InForceSmallIcons));
}

TSharedPtr<FToolBarBuilder> UUTBVerticalToolBarTabUI::GetBuilder(TSharedPtr<const FUICommandList> InCommandList,
	FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender, const bool InForceSmallIcons)
{
	return MakeShareable(new FVerticalToolBarBuilder(InCommandList,InCustomization,InExtender,InForceSmallIcons));
}

TSharedPtr<FToolBarBuilder> UUTBSlimHorizontalToolBarTabUI::GetBuilder(TSharedPtr<const FUICommandList> InCommandList,
	FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender, const bool InForceSmallIcons)
{
	return MakeShareable(new FSlimHorizontalToolBarBuilder(InCommandList,InCustomization,InExtender,InForceSmallIcons));
}
