// Copyright Epic Games, Inc. All Rights Reserved.


#include "UTBFactory.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "UserToolBoxBaseBlueprint.h"
#include "UTBEditorBlueprint.h"
#include "Kismet2/SClassPickerDialog.h"
#include "ClassViewerFilter.h"
#include "IconsTracker.h"
#include "UTBBaseTab.h"
#include "Widgets/Layout/SScrollBox.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Text/STextBlock.h"
#include "Editor.h"

UUTBEditorUtilityBlueprintFactory::UUTBEditorUtilityBlueprintFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UUTBEditorBlueprint::StaticClass();
}
UObject* UUTBEditorUtilityBlueprintFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName,
	EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(InClass->IsChildOf(UBlueprint::StaticClass()));


		return FKismetEditorUtilities::CreateBlueprint(UUserToolBoxBaseBlueprint::StaticClass(), InParent, InName, BPTYPE_Normal, UUTBEditorBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass());

}

bool UUTBEditorUtilityBlueprintFactory::CanCreateNew() const
{
	return true;
}



class FUTBCommandFactoryFilter : public IClassViewerFilter
{
public:
	TSet< const UClass* > AllowedChildOfClasses;

	TSet< const UClass*> DisallowedChildOfClasses;

	bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		if (DisallowedChildOfClasses.Num() == 0 && AllowedChildOfClasses.Num() == 0)
		{
			return true;
		}

	
		bool IsOk=InFilterFuncs->IfInChildOfClassesSet(AllowedChildOfClasses, InClass) != EFilterReturn::Failed;
		IsOk=IsOk && InFilterFuncs->IfInChildOfClassesSet(DisallowedChildOfClasses, InClass) == EFilterReturn::Failed; 
		return IsOk; 
			
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		if (DisallowedChildOfClasses.Num() == 0 && AllowedChildOfClasses.Num() == 0)
		{
			return true;
		}

		return InFilterFuncs->IfInChildOfClassesSet(AllowedChildOfClasses, InUnloadedClassData) != EFilterReturn::Failed
			&& (InFilterFuncs->IfInChildOfClassesSet(DisallowedChildOfClasses, InUnloadedClassData) == EFilterReturn::Failed);;
	}
};



UUTBCommandFactory::UUTBCommandFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UUTBBaseCommand::StaticClass();
	ParentClass= nullptr;
}
UObject* UUTBCommandFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags,
	UObject* Context, FFeedbackContext* Warn)
{
	UUTBBaseCommand* Command = NewObject<UUTBBaseCommand>(InParent, ParentClass, InName, Flags);
	return Command;
}

bool UUTBCommandFactory::CanCreateNew() const
{
	return true;
}

bool UUTBCommandFactory::ConfigureProperties()
{
	TArray<UClass*> ValidClasses;
	for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
	{
		// Skip SKEL and REINST classes.
		if (ClassIterator->GetName().StartsWith(TEXT("SKEL_")) || ClassIterator->GetName().StartsWith(TEXT("REINST_")))
		{
			continue;
		}
		if (ClassIterator->IsChildOf(UUTBBaseCommand::StaticClass()) && *ClassIterator != UUTBBaseCommand::StaticClass() && *ClassIterator != UUserToolBoxBaseBlueprint::StaticClass())
		{
				ValidClasses.Add(*ClassIterator);
		}
	}
	TMap<FString,TArray<UClass*>> CDOPerCategory;
	for (UClass* ValidClass:ValidClasses)
	{
		UUTBBaseCommand* CDO =Cast<UUTBBaseCommand>(ValidClass->GetDefaultObject());
		if (IsValid(CDO))
		{
			CDOPerCategory.FindOrAdd(CDO->Category).Add(ValidClass);
		}
	}
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(FText::FromString("Pick a parent command"))
		.ClientSize(FVector2D(200.f, 300.0f))
		.IsTopmostWindow(true);

	

	struct TreeNode
	{
		bool bIsCategory= false;
		UClass*		  CommandClass;
		FString			  Name;
		TArray<TSharedPtr<TreeNode>> Children;
	};
	TArray<TSharedPtr<TreeNode>> Tree;
	for (TTuple<FString,TArray<UClass*>> Category:CDOPerCategory)
	{
		FString CategoryFullName=Category.Key;
		if (CategoryFullName.IsEmpty())
		{
			CategoryFullName="Uncategorized";
		}
		
		TArray<FString> CategoriesSplit;
		CategoryFullName.ParseIntoArray(CategoriesSplit,TEXT("|"),1);
		TArray<TSharedPtr<TreeNode>>* CurrentArray=&Tree;
		while (!CategoriesSplit.IsEmpty())
		{
			TSharedPtr<TreeNode>* CurrentNode= CurrentArray->FindByPredicate([&CategoriesSplit](TSharedPtr<TreeNode> Node)
			{
				return CategoriesSplit[0].Compare(Node->Name)==0;
			});
			if (CurrentNode==nullptr)
			{
				TSharedPtr<TreeNode> NewNode=CurrentArray->Add_GetRef(MakeShared<TreeNode>());
				NewNode->bIsCategory=true;
				NewNode->Name=CategoriesSplit[0];
				CurrentNode=&NewNode;
			}
			
			CurrentArray=&(*CurrentNode)->Children;
			CategoriesSplit.RemoveAt(0);
		}
		for (UClass* ValidClass:Category.Value)
		{
			TSharedPtr<TreeNode> NewNode=CurrentArray->Add_GetRef(MakeShared<TreeNode>());
			NewNode->Name=ValidClass->GetName();
			NewNode->bIsCategory=false;
			NewNode->CommandClass=ValidClass;
		}
	}
	UClass* SelectedValue;
	Window->SetContent(
		SNew(SScrollBox)
		+SScrollBox::Slot()
		[
			SNew(STreeView<TSharedPtr<TreeNode>>)
			.TreeItemsSource(&Tree)
			.OnGenerateRow(STreeView<TSharedPtr<TreeNode>>::FOnGenerateRow::CreateLambda([](TSharedPtr<TreeNode> Node,const TSharedRef<STableViewBase>& OwnerTable)
			{
				TSharedPtr<STextBlock> TextBlock;
				if (Node.IsValid())
				{
					SAssignNew(TextBlock,STextBlock).Text(FText::FromString(Node->Name));
				}
				else
				{
					SAssignNew(TextBlock,STextBlock).Text(FText::FromString("Invalid node"));
				}
				return SNew(STableRow<TSharedPtr<TreeNode>>,OwnerTable)
				[
					TextBlock.ToSharedRef()
				];
			}))
			.OnGetChildren(STreeView<TSharedPtr<TreeNode>>::FOnGetChildren::CreateLambda([](TSharedPtr<TreeNode> ParentNode,TArray<TSharedPtr<TreeNode>>& Children)
			{
				Children=ParentNode->Children;
			}))
			.OnMouseButtonClick(STreeView<TSharedPtr<TreeNode>>::FOnMouseButtonClick::CreateLambda([&SelectedValue,&Window](TSharedPtr<TreeNode> Node)
			{
				if (!Node->bIsCategory)
				{
					SelectedValue=Node->CommandClass;
					Window->RequestDestroyWindow();
				}
			}))
		]
		);
	
	GEditor->EditorAddModalWindow(Window);
	if (IsValid(SelectedValue))
	{
		ParentClass=SelectedValue;
	}
	else
	{
		ParentClass=nullptr;
	}
	return ParentClass!=nullptr;
}


UUTBCommandTabFactory::UUTBCommandTabFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UUserToolBoxBaseTab::StaticClass();
}
UObject* UUTBCommandTabFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags,
	UObject* Context, FFeedbackContext* Warn)
{
	UUserToolBoxBaseTab* Tab = NewObject<UUserToolBoxBaseTab>(InParent, UUserToolBoxBaseTab::StaticClass(), InName, Flags);
	return Tab;
}

bool UUTBCommandTabFactory::CanCreateNew() const
{
	return true;
}

UIconTrackerFactory::UIconTrackerFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UIconsTracker::StaticClass();
}
UObject* UIconTrackerFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName,
	EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return  NewObject<UIconsTracker>(InParent, UIconsTracker::StaticClass(), InName, Flags);
}

bool UIconTrackerFactory::CanCreateNew() const
{
	return true;
}
