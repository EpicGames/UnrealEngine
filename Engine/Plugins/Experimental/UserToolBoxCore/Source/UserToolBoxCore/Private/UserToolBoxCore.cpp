// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserToolBoxCore.h"
#include "UTBBaseCommand.h"
#include "Customization/BaseCommandCustomization.h"
#include "UObject/CoreRedirects.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FUserToolBoxCoreModule"

void FUserToolBoxCoreModule::StartupModule()
{

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.RegisterCustomClassLayout(UUTBBaseCommand::StaticClass()->GetFName(),FOnGetDetailCustomizationInstance::CreateStatic(&FBaseCommandCustomization::MakeInstance));
	
}

void FUserToolBoxCoreModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE
DEFINE_LOG_CATEGORY(LogUserToolBoxCore);
IMPLEMENT_MODULE(FUserToolBoxCoreModule, UserToolBoxCore)