// Copyright Epic Games, Inc. All Rights Reserved.


#include "UserToolBoxFunctionLibrary.h"
#include "UserToolBoxStyle.h"
#include "Styling/SlateStyleRegistry.h"

const FSlateBrush UUserToolBoxFunctionLibrary::DefaultBrush=FSlateBrush();
const FSlateBrush& UUserToolBoxFunctionLibrary::GetBrushById(FString Name)
{
	const FSlateBrush* Ret=FUserToolBoxStyle::Get().GetBrush(FName(Name));
	if (Ret==nullptr)
	{
		return DefaultBrush;
	}
	return *Ret;
}

TArray<FName> UUserToolBoxFunctionLibrary::GetAllSlateStyle()
{
	TArray<FName> Names;

	FSlateStyleRegistry::IterateAllStyles(
		[&](const ISlateStyle& Style)
		{
			Names.Add(Style.GetStyleSetName());
			return true;
		}
	);
	return Names;
}

const FSlateBrush& UUserToolBoxFunctionLibrary::GetBrushByStyleAndId(FName StyleName, FName Id)
{
	
	
	const ISlateStyle* Style=	FSlateStyleRegistry::FindSlateStyle(StyleName);
	if (Style==nullptr)
	{
		return DefaultBrush; 
	}
	const FSlateBrush*	Brush= Style->GetBrush(Id);
	if (Brush==nullptr)
	{
		return DefaultBrush;
	}
	return *Brush;
}
