// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UserToolBoxFunctionLibrary.generated.h"

/**
 * 
 */
UCLASS()
class UUserToolBoxFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	static const FSlateBrush	DefaultBrush;
	
	UFUNCTION(BlueprintCallable, Category=" User Toolbox Library")
	static const FSlateBrush& GetBrushById(FString Name);

	UFUNCTION(BlueprintCallable, Category=" User Toolbox Library")
	static TArray<FName> GetAllSlateStyle();

	UFUNCTION(BlueprintCallable, Category=" User Toolbox Library")
	static const FSlateBrush& GetBrushByStyleAndId(FName StyleName, FName Id);

	
};
