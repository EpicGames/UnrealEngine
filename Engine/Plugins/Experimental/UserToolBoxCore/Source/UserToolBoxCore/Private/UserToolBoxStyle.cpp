// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserToolBoxStyle.h"
#include "Styling/SlateStyle.h"
#include "FileCache.h"
#include "Styling/SlateStyleRegistry.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"

#define RootToContentDir Style->RootToContentDir

TSharedPtr< FSlateStyleSet > FUserToolBoxStyle::StyleInstance = nullptr;
TArray<FString> FUserToolBoxStyle::ExternalBrushIds;
TMap<FString, TUniquePtr<FSlateImageBrush>> FUserToolBoxStyle::ExternalBrushes;

void FUserToolBoxStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FUserToolBoxStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FUserToolBoxStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("UserToolBoxStyle"));
	return StyleSetName;
}

const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon15x15(16.0f, 16.0f);
const FVector2D Icon30x30(16.0f, 16.0f);
const FVector2D Icon60x60(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);

TSharedRef< FSlateStyleSet > FUserToolBoxStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("UserToolBoxStyle"));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin("UserToolBoxCore")->GetBaseDir() / TEXT("Resources"));
	Style->Set("Palette.FirstHeader",new FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(4.0, 4.0, 0.0, 0.0)));
	Style->Set("Palette.LastHeader",new FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0, 0.0, 4.0, 4.0)));
	Style->Set("Palette.Header",new FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0, 0.0, 0.0, 0.0)));
	Style->Set("Palette.UniqueHeader",new FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(4.0, 4.0, 4.0, 4.0)));
	Style->Set("Palette.Body",new FSlateRoundedBoxBrush(FStyleColors::Recessed, FVector4(0.0, 0.0, 0.0, 0.0)));
	
	
	return Style;
}

void FUserToolBoxStyle::AddExternalImageBrushes(const TArray<FIconInfo>& IconInfos)
{
	if (StyleInstance==nullptr)
	{
		return;
	}

	bool bChanged = false;

	for (const FIconInfo& Info:IconInfos)
	{
		const FString Key = FString::Printf(TEXT("%s_%s"), *Info.Path, *Info.IconSize.ToString());

		TUniquePtr<FSlateImageBrush>* FoundBrush = ExternalBrushes.Find(Key);

		if (!FoundBrush)
		{
			TUniquePtr<FSlateImageBrush> Brush = MakeUnique<FSlateImageBrush>(Info.Path, Info.IconSize);
			ExternalBrushes.Add(Key, MoveTemp(Brush));
			bChanged = true;
		}

		FoundBrush = ExternalBrushes.Find(Key);

		if (FoundBrush)
		{
			StyleInstance->Set(*Info.Id, FoundBrush->Get());

			ExternalBrushIds.Add(Info.Id);
		}
	}

	if (bChanged)
	{
		ReloadTextures();
	}
}

TArray<FString> FUserToolBoxStyle::GetAvailableExternalImageBrushes()
{
	return ExternalBrushIds;
}

void FUserToolBoxStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		if (auto Renderer = FSlateApplication::Get().GetRenderer())
		{
			Renderer->ReloadTextureResources();
		}
	}
}

const ISlateStyle& FUserToolBoxStyle::Get()
{
	return *StyleInstance;
}

#undef RootToContentDir
