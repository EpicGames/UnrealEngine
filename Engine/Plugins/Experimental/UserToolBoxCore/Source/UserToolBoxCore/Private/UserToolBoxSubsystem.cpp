// Copyright Epic Games, Inc. All Rights Reserved.


#include "UserToolBoxSubsystem.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UTBBaseTab.h"
#include "ToolMenus.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Modules/ModuleManager.h"
#include "AssetToolsModule.h"
#include "WidgetBlueprintEditor.h"
#include "AssetTypeActions_UTBEditorUtilityBlueprint.h"
#include "EditorModeRegistry.h"
#include "AssetTypeActions/AssetTypeActions_Blueprint.h"
#include "WidgetBlueprint.h"
#include "UserToolBoxStyle.h"
#include "UTBBaseUITab.h"
#include "IconsTracker.h"
#include "LevelEditor.h"
#include "NetworkMessage.h"
#include "SLevelViewport.h"
#include "StatusBarSubsystem.h"
#include "UserToolBoxCore.h"
#include "WidgetDrawerConfig.h"
#include "Dialog/SCustomDialog.h"
#include "Editor/UTBTabEditor.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "SVerticalTextBlock.h"
#include "Editor/UTBEditorCommands.h"
#include "Interfaces/IMainFrameModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "HAL/FileManager.h"
class IMainFrameModule;
EAssetTypeCategories::Type UUserToolboxSubsystem::AssetCategory=EAssetTypeCategories::Basic;

class FAssetTypeActions_UTBBaseTab : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions interface
	virtual FText GetName() const override {return FText::FromString("User ToolBox Tab");};
	virtual FColor GetTypeColor() const override{return FColor(169, 50, 0);};
	virtual UClass* GetSupportedClass() const override {return UUserToolBoxBaseTab::StaticClass();};
	virtual uint32 GetCategories() override{return UUserToolboxSubsystem::AssetCategory;};
	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects,
		TSharedPtr<IToolkitHost> EditWithinLevelEditor) override
	{
		TArray<UUserToolBoxBaseTab*> Tabs;
		for (UObject* Object:InObjects)
		{
			UUserToolBoxBaseTab* Tab=Cast<UUserToolBoxBaseTab>(Object);
			if (IsValid(Tab))
			{
				Tabs.Add(Tab);
			}
		}
		
		for (UUserToolBoxBaseTab* Tab:Tabs)
		{
			TSharedRef< FUTBTabEditor > NewDataTableEditor( new FUTBTabEditor() );
			NewDataTableEditor->InitUTBTabEditor( EToolkitMode::Standalone, EditWithinLevelEditor, Tab );
		}
	}
	// End of IAssetTypeActions interface
};


class FAssetTypeActions_UIconTracker : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions interface
	virtual FText GetName() const override {return FText::FromString("Icon Tracker");};
	virtual FColor GetTypeColor() const override{return FColor(100, 100, 0);};
	virtual UClass* GetSupportedClass() const override {return UIconsTracker::StaticClass();};
	virtual uint32 GetCategories() override{return UUserToolboxSubsystem::AssetCategory;};
	// End of IAssetTypeActions interface
};
FAutoConsoleCommand RefreshIcons = FAutoConsoleCommand(
	TEXT("UserToolbox.RefreshIcons"),
	TEXT("RefreshIcons"),
	FConsoleCommandDelegate::CreateLambda([]() {
		GEditor->GetEditorSubsystem<UUserToolboxSubsystem>()->RefreshIcons();
	}));
FAutoConsoleCommand RefreshLevelViewportOverlay = FAutoConsoleCommand(
	TEXT("UserToolbox.LevelViewportOverlay"),
	TEXT("RefreshIcons"),
	FConsoleCommandDelegate::CreateLambda([]() {
		GEditor->GetEditorSubsystem<UUserToolboxSubsystem>()->UpdateLevelViewportWidget();
	}));
FAutoConsoleCommand RefreshDrawer = FAutoConsoleCommand(
	TEXT("UserToolbox.RefreshDrawer"),
	TEXT("Refresh the Drawer"),
	FConsoleCommandDelegate::CreateLambda([]() {
		GEditor->GetEditorSubsystem<UUserToolboxSubsystem>()->RegisterDrawer();
	}));



void UUserToolboxSubsystem::RegisterTabData()
{
	RefreshIcons();
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Window");
	FToolMenuSection& Section = Menu->FindOrAddSection("UserToolBox");
	Section.AddSubMenu("UserToolBox", FText::FromString("UserToolBox"), FText::FromString("Custom toolbox made by users"),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
		{
			TArray<FAssetData> TabList=GetAvailableTabList();
			for (const FAssetData& TabAssetData :TabList)
			{
				// need to handle collision
				bool IsActive=TabAssetData.GetTagValueRef<bool>("bIsVisibleInWindowsMenu");
				if (!IsActive)
				{
					continue;
				}
				FString TabName=TabAssetData.GetTagValueRef<FString>("Name");
				MenuBuilder.AddMenuEntry(FText::FromString(TabName),FText::FromString(TabName),FSlateIcon(),
				FUIAction(
		FExecuteAction::CreateLambda([this,TabAssetData,TabName]()
					{
						FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FName(TabName),FOnSpawnTab::CreateLambda([this,TabAssetData,TabName](const FSpawnTabArgs& arg)
						{
								
							return SNew(SDockTab)
							.TabRole(NomadTab)
							[
							GenerateTabUI(TabAssetData).ToSharedRef()
							];
							
						}));
						FGlobalTabmanager::Get()->TryInvokeTab(FName(TabName));
						}),
					FCanExecuteAction(),
					FIsActionChecked()
					)
				)
				;
			}
		}));
	UpdateLevelViewportWidget();
}
TArray<FAssetData> UUserToolboxSubsystem::GetAvailableTabList()
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	FARFilter Filter;
	Filter.bIncludeOnlyOnDiskAssets = false;
	Filter.ClassPaths.Add(UUserToolBoxBaseTab::StaticClass()->GetClassPathName());
	Filter.bRecursivePaths = true;
	Filter.bRecursiveClasses = true;
	TArray<FAssetData> AssetDatas;
	AssetRegistry.GetAssets(Filter,AssetDatas);
	TArray<FAssetData> TabList;
	for (FAssetData AssetData:AssetDatas)
	{
		UUserToolBoxBaseTab* Tab=Cast<UUserToolBoxBaseTab>(AssetData.GetAsset());
		if (Tab==nullptr)
		{
			UE_LOGF(LogTemp,Warning,"Unable to load %ls",*AssetData.PackageName.ToString())
			continue;
		}
		TabList.Add(AssetData);
	}
	return TabList;
}


bool UUserToolboxSubsystem::PickAnIcon(FString& OutValue)
{
	IconOptions.Empty();
	IconOptions.Add(MakeShareable(new FString("None")));
	TArray<FString> BrushIds=FUserToolBoxStyle::GetAvailableExternalImageBrushes();

	FString CurrentValue="";
	for (const FString& BrushId:BrushIds)
	{
		IconOptions.Add(MakeShareable(new FString(BrushId)));
	}
	IconOptions.Sort([](const TSharedPtr<FString>& A, const TSharedPtr<FString>& B)
	{
		return *A<*B;
	});

	TSharedRef<SCustomDialog> OptionsWindow =
	SNew(SCustomDialog)
	.Title(FText::FromString("Icons"))
	
	.Buttons
	(
		{
		SCustomDialog::FButton(FText::FromString("Ok")),
		SCustomDialog::FButton(FText::FromString("Cancel")),
		}
	)
	.Content()
	[
	SNew(SListView<TSharedPtr<FString>>)
		.ListItemsSource(&IconOptions)
		.OnSelectionChanged(SListView<TSharedPtr<FString>>::FOnSelectionChanged::CreateLambda([&CurrentValue](TSharedPtr<FString> Value, ESelectInfo::Type Type)
		{
			if (Value->Compare("None")==0)
			{
				CurrentValue="";
			}
			else
			{
				CurrentValue=*Value;
			}
		}))
		.OnGenerateRow_Lambda([](TSharedPtr<FString> Node,const TSharedRef<STableViewBase>& OwnerTable)
		{

			TSharedPtr<SHorizontalBox> HBox=
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock).Text(FText::FromString(*Node))
				];
				
			if ((*Node)!="None")
			{
				HBox->AddSlot()
            				.HAlign(HAlign_Right)
            				[
            					SNew(SImage)
            					.Image(FUserToolBoxStyle::Get().GetBrush(FName(*Node)))
            				]	;
			}
			return SNew(STableRow<TSharedPtr<FString>>,OwnerTable)
			[
				HBox.ToSharedRef()
			];
		})
	];
	
	const int32 PressedButtonIdx = OptionsWindow->ShowModal();
	if (PressedButtonIdx != 0)
	{
		return false;
	}
	OutValue=CurrentValue;
	return true;
}

void UUserToolboxSubsystem::RefreshIcons()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	FARFilter Filter;
	Filter.bRecursiveClasses=true;
	Filter.ClassPaths.Add(UIconsTracker::StaticClass()->GetClassPathName());
	TArray<FAssetData> AssetDatas;
	AssetRegistry.GetAssets(Filter,AssetDatas);
	TArray<FIconInfo> IconDatas;
	for (const FAssetData& AssetData:AssetDatas)
	{
		UIconsTracker* Asset=Cast<UIconsTracker>(AssetData.GetAsset());
		if (IsValid(Asset))
		{

			FString Root;
			if (AssetData.PackageName.ToString().StartsWith("/Game/"))
			{// current project
				Root=FPaths::ProjectDir();
			}
			else if ( AssetData.PackageName.ToString().StartsWith("/Engine/") )
			{
				Root=FPaths::EngineDir();
			}
			else
			{
				FString Tmp=AssetData.PackageName.ToString().RightChop(1);
				int Index=0;
				Tmp.FindChar('/',Index);
				FString PluginName=Tmp.Left(Index);
				TSharedPtr<IPlugin> Plugin=IPluginManager::Get().FindPlugin(PluginName);
				if (Plugin!=nullptr)
				{
					Root=Plugin->GetBaseDir();
				}
					
			}
			if (!Root.IsEmpty())
			{
				for (const FIconFolderInfo& IconFolderInfo:Asset->IconFolderInfos)
				{
					
					TArray<FString> IconPaths;
					FString DirPath;
					if (FPaths::IsRelative(IconFolderInfo.FolderPath.Path))
					{
						DirPath=Root / IconFolderInfo.FolderPath.Path ;
					}
					else
					{
						DirPath=IconFolderInfo.FolderPath.Path;
					}
					
					if (!FPaths::DirectoryExists(DirPath))
					{
						UE_LOGF(LogUserToolBoxCore,Error," directory doesn't exist: %ls",*DirPath);
						continue;
					}
					
					IFileManager::Get().FindFiles(IconPaths, *DirPath, TEXT(".png"));

					for (const FString& IconPath:IconPaths)
					{
						if (FPaths::FileExists(DirPath / IconPath))
						{
							FIconInfo Info;
							Info.Id=Asset->PrefixId+IconFolderInfo.PrefixId+FPaths::GetBaseFilename(IconPath);
							Info.Path=DirPath / IconPath;
							Info.IconSize=IconFolderInfo.IconSize;
							IconDatas.Add(Info);
					
						}	
					}
				}	
			}
		}
	}

	FUserToolBoxStyle::AddExternalImageBrushes(IconDatas);
}
TSharedPtr<SWidget> UUserToolboxSubsystem::GenerateTabUI(const FAssetData Data, TSubclassOf<UUTBDefaultUITemplate> Ui,const FUITemplateParameters& Parameters)
{
	if (!Data.IsValid())
	{
		return SNullWidget::NullWidget;
	}
	UUserToolBoxBaseTab* Tab= Cast<UUserToolBoxBaseTab>(Data.GetAsset());
	if (Tab==nullptr)
	{
		return SNullWidget::NullWidget;
	}
	RefreshIcons();
	if (Data.IsValid())
	{
		return SNew(SUserToolBoxTabWidget)
		.Tab(Data.GetAsset())
		.UIOverride(Ui)
		.UIParameters(Parameters);
	}
	return SNew(SDockTab);	
	
}
void UUserToolboxSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	AssetCategory = AssetTools.RegisterAdvancedAssetCategory(FName(TEXT("UserToolBox")), FText::FromString("User Tool Box"));
	AssetTools.RegisterAssetTypeActions(MakeShared<FAssetTypeActions_UTBEditorBlueprint>());
	AssetTools.RegisterAssetTypeActions(MakeShared<FAssetTypeActions_UTBBaseTab>());
	AssetTools.RegisterAssetTypeActions(MakeShared<FAssetTypeActions_UIconTracker>());
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	OnFileLoadHandle=AssetRegistry.OnFilesLoaded().AddLambda([this](){RegisterTabData();});

	AssetRegistry.OnAssetAdded().AddUObject(this, &UUserToolboxSubsystem::OnAssetChanged);
	AssetRegistry.OnAssetRemoved().AddUObject(this, &UUserToolboxSubsystem::OnAssetChanged);
	AssetRegistry.OnAssetUpdatedOnDisk().AddUObject(this, &UUserToolboxSubsystem::OnAssetChanged);

	FUserToolBoxStyle::Initialize();
	FUTBEditorCommands::Register();
	
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	LevelEditorModule.OnTabManagerChanged().AddUObject(this, &UUserToolboxSubsystem::ReinitializeUIs);
}

TSharedRef<SDockTab> UUserToolboxSubsystem::CreateTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const FAssetData* TabAssetData =
		AvailableTabList.FindByPredicate([SpawnTabArgs](const FAssetData& Tab)
		{
			return Tab.GetTagValueRef<FString>("Name") == SpawnTabArgs.GetTabId().ToString();
		});
	
	return SNew(SDockTab)
			.TabRole(NomadTab)
			[
				GEditor->GetEditorSubsystem<UUserToolboxSubsystem>()->GenerateTabUI(*TabAssetData).ToSharedRef()
			];
}

void UUserToolboxSubsystem::ReinitializeUIs()
{
	AvailableTabList = GetAvailableTabList();
	for (const FAssetData& TabAssetData : AvailableTabList)
	{
		FString TabName=TabAssetData.GetTagValueRef<FString>("Name");
		if (RegisteredDrawer.Contains(FName(TabName)))
		{
			continue;	
		}
		
		if (TabAssetData.GetTagValueRef<bool>("bIsVisibleInDrawer"))
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
			if (const TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
			{
				LevelEditorTabManager->RegisterTabSpawner(FName(TabName), FOnSpawnTab::CreateUObject(this, &UUserToolboxSubsystem::CreateTab));
			}
		}
	}
}

void UUserToolboxSubsystem::Deinitialize()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	AssetRegistry.OnAssetAdded().RemoveAll(this);
	AssetRegistry.OnAssetRemoved().RemoveAll(this);
	AssetRegistry.OnAssetUpdatedOnDisk().RemoveAll(this);

	if (OnFileLoadHandle.IsValid())
	{
		
		AssetRegistry.OnFilesLoaded().Remove(OnFileLoadHandle);
	}
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveLevelViewport();
	if (ActiveLevelViewport.IsValid())
	{
		if (LevelViewportOverlayWdget.IsValid())
		{
			ActiveLevelViewport->RemoveOverlayWidget(LevelViewportOverlayWdget.ToSharedRef());
			LevelViewportOverlayWdget.Reset();
		}
	}
	if (LevelViewportOverlayWdget.IsValid())
	{
		LevelViewportOverlayWdget.Reset();
	}
	Super::Deinitialize();
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		const FUTBEditorCommands& Commands = FUTBEditorCommands::Get();

		IMainFrameModule& MainFrame = FModuleManager::Get().LoadModuleChecked<IMainFrameModule>("MainFrame");

		FUICommandList& ActionList = *MainFrame.GetMainFrameCommandBindings();

		ActionList.UnmapAction(Commands.RenameSection);

	}
	
	FUTBEditorCommands::Unregister();
}

void UUserToolboxSubsystem::RegisterDrawer()
{
	UStatusBarSubsystem* StatusBarSubsystem=GEditor->GetEditorSubsystem<UStatusBarSubsystem>();
	for (FName TabToUnregister:RegisteredDrawer)
	{
		//TODO find a way to clean the drawerbar
	}
	if (IsValid(StatusBarSubsystem))
	{
		TArray<FAssetData> TabList=GetAvailableTabList();
		for (const FAssetData& TabAssetData :TabList)
		{
			FString TabName=TabAssetData.GetTagValueRef<FString>("Name");
			if (RegisteredDrawer.Contains(FName(TabName)))
			{
				continue;	
			}
			bool IsVisible=TabAssetData.GetTagValueRef<bool>("bIsVisibleInDrawer");
			if (IsVisible)
			{
				FName TabNameAsFName(TabName);
				FWidgetDrawerConfig WidgetDrawerConfig(TabNameAsFName);
				WidgetDrawerConfig.ButtonText=FText::FromString(TabName);
				WidgetDrawerConfig.ToolTipText=FText::FromString(TabName);
				WidgetDrawerConfig.GetDrawerContentDelegate=FOnGetContent::CreateLambda([this,TabAssetData]()
				{
					return GenerateTabUI(TabAssetData, nullptr).ToSharedRef();
					
				});
				WidgetDrawerConfig.OnDrawerDismissedDelegate=FOnStatusBarDrawerDismissed::CreateLambda([]( const TSharedPtr<SWidget>& widget)
				{
					UE_LOGF(LogTemp, Warning, "Dismissed")
				});
				WidgetDrawerConfig.OnDrawerOpenedDelegate=FOnStatusBarDrawerOpened::CreateLambda([](FName StatusBarName)
				{
					UE_LOGF(LogTemp, Warning, "openned %ls",*StatusBarName.ToString());
				});
				RegisteredDrawer.Add(FName(TabName));
				StatusBarSubsystem->RegisterDrawer("LevelEditor.StatusBar",MoveTemp(WidgetDrawerConfig));				
			}
				
		}
		
		
	}
}

void UUserToolboxSubsystem::UpdateLevelViewportWidget()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveLevelViewport();
	RegisterDrawer();
	TArray<FAssetData> TabToAdd;
	if (ActiveLevelViewport.IsValid())
	{
		if (LevelViewportOverlayWdget.IsValid())
		{
			ActiveLevelViewport->RemoveOverlayWidget(LevelViewportOverlayWdget.ToSharedRef());
			LevelViewportOverlayWdget.Reset();
		}
		TArray<FAssetData> TabList=GetAvailableTabList();
		for (const FAssetData& TabAssetData :TabList)
		{
			// need to handle collision
			bool IsVisible=TabAssetData.GetTagValueRef<bool>("bIsVisibleInViewportOverlay");
			if (IsVisible)
			{
				TabToAdd.Add(TabAssetData);
			}
		}
		if (!TabToAdd.IsEmpty())
		{
			TSharedPtr<SVerticalBox> VerticalBox;
			LevelViewportOverlayWdget=
				SNew(SConstraintCanvas)
				 +SConstraintCanvas::Slot()
				.Anchors(FAnchors(0.0f,.0f))
				.Alignment(FVector2d(0.0,0.0f))
				.Offset(FMargin(0.0f,138.0f,0.0,0.0f))
				.AutoSize(true)
				 [
					 SNew(SBox)
					 .MaxDesiredWidth(250)
					 [
						 SAssignNew(VerticalBox,SVerticalBox)
				 	]
				 ];
			int Index=0;
			for (FAssetData& TabAssetData:TabToAdd)
			{
				FName HeaderStyle="Palette.Header";
				if (TabToAdd.Num()==1)
				{
					HeaderStyle="Palette.UniqueHeader";	
				}
				else
				{
					if (Index==0)
					{
						HeaderStyle="Palette.FirstHeader";
					}
					if (Index==TabToAdd.Num()-1)
					{
						HeaderStyle="Palette.LastHeader";
					}
				}
				FString TabName=TabAssetData.GetTagValueRef<FString>("Name");
				FUITemplateParameters Params;
				Params.bForceHideSettingsButton=true;
				Params.bPrefixSectionWithTabName=false;
				TSharedPtr<FCurveSequence> CurveSequence= MakeShared<FCurveSequence>(0,.3,ECurveEaseFunction::CubicIn);
				const FSlateRenderTransform Rotate90(FQuat2D(FMath::DegreesToRadians(90.f)));
				VerticalBox->AddSlot()
				 .AutoHeight()
				 .HAlign(HAlign_Fill)
				 .VAlign(VAlign_Top)
				[
					SNew(SBox)
					.MinDesiredHeight(100)
					[
						
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.HAlign(HAlign_Left)
							.AutoWidth()
							[
								SNew(SBox)
								.WidthOverride(20)
								.Clipping(EWidgetClipping::Inherit)
								[
									SNew(SButton)
									.ContentPadding(FMargin(5,10,0,10))
									.Clipping(EWidgetClipping::Inherit)
									.OnClicked_Lambda([CurveSequence,this]()
									{
										if (CurveSequence->GetSequenceTime()<0.1)
										{
											CurveSequence->Play(LevelViewportOverlayWdget->AsShared());
										}
										if (CurveSequence->GetSequenceTime()>0.9)
										{
											CurveSequence->PlayReverse(LevelViewportOverlayWdget->AsShared());
										}
										return FReply::Handled();
									})
									[
										SNew(SVerticalTextBlock)
										.Clipping(EWidgetClipping::Inherit)
										.Text(FText::FromString(TabName))
										.Font(FAppStyle::Get().GetFontStyle("BoldFont"))
										.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.Foreground"))

									]
								]
							]
							+SHorizontalBox::Slot()
							.HAlign(HAlign_Fill)
							[
								SNew(SBorder)
								.Padding(0)
								.DesiredSizeScale_Lambda([CurveSequence]()
								{
									return FVector2d(CurveSequence->GetLerp(),1);
								})
								.Visibility_Lambda([CurveSequence]()
								{
									return CurveSequence->GetLerp()<0.1?EVisibility::Collapsed:EVisibility::Visible;
								})
								.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
								
								.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.Recessed"))
								[
									GenerateTabUI(TabAssetData, UUTBVerticalToolBarTabUI::StaticClass(),Params).ToSharedRef()
								]
							]
					]
				];
				VerticalBox->AddSlot()
				.AutoHeight()
				[
					SNew(SSpacer).Size(FVector(20.0,20.0,20.0))
				];
				Index++;
			}
			if (LevelViewportOverlayWdget.IsValid())
			{
				ActiveLevelViewport->AddOverlayWidget(LevelViewportOverlayWdget.ToSharedRef());	
			}
		}
	}
}

void UUserToolboxSubsystem::OnAssetChanged(const FAssetData& InAssetData)
{
	if (InAssetData.AssetClassPath != UIconsTracker::StaticClass()->GetClassPathName())
	{
		return;
	}

	UIconsTracker* Asset = Cast<UIconsTracker>(InAssetData.GetAsset());
	if (IsValid(Asset))
	{
		RefreshIcons();
	}
}
