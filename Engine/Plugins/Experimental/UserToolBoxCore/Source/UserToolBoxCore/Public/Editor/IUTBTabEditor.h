// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"
/**
 * 
 */
class USERTOOLBOXCORE_API IUTBTabEditor : public FAssetEditorToolkit
{

	
};
