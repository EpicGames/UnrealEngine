// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseTab.h"
#include "Widgets/Views/STreeView.h"
/**
 * 
 */
DECLARE_DELEGATE_OneParam(FOnCommandSelectedChanged, UClass*)
class USERTOOLBOXCORE_API SCommandPickMenuWidget : public SCompoundWidget
{
	struct FCommandTreeNode
	{
		bool bIsCategory= false;
		UClass*									Class;
		FString									Name;
		TArray<TSharedPtr<FCommandTreeNode>>	Children;
		TArray<TSharedPtr<FCommandTreeNode>>	FilteredChildren;
	};


public:
	SLATE_BEGIN_ARGS( SCommandPickMenuWidget )
		 
			:_OnCommandSelectionChanged()
		{
		
		}
	SLATE_EVENT(FOnCommandSelectedChanged, OnCommandSelectionChanged)
		/** Sets the text content for this editable text widget */
		
		
	SLATE_END_ARGS()


	SCommandPickMenuWidget();
	~SCommandPickMenuWidget();

	void Construct( const FArguments& InArgs );

	
private:
	void ReconstructCommandTree();

	void UpdateFilteredTreeNodes(FString Text);
	void OnSetExpansionRecursive(TSharedPtr<FCommandTreeNode> InTreeNode, bool bInIsItemExpanded);


	FOnCommandSelectedChanged OnCommandSelectionChanged;
	TArray<TSharedPtr<FCommandTreeNode>> CommandTree;
	TArray<TSharedPtr<FCommandTreeNode>> CommandTreeAllNodes;
	
	TMap<TSharedPtr<FCommandTreeNode>,TArray<TSharedPtr<FCommandTreeNode>>> LeafsWithParentHierarchy;// reverse tree for seaching
	
	TSharedPtr<STreeView<TSharedPtr<FCommandTreeNode>>> TreeViewPtr;

	UClass* Selection;
};

