// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseTab.h"
#include "IDetailsView.h"
/**
 * 
 */
DECLARE_DELEGATE_OneParam(FOnObjectPropertyModified,UUTBBaseCommand*);
class USERTOOLBOXCORE_API SCommandDetailsView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SCommandDetailsView )
		: _Command() 
		{
		
		}

		/** Sets the text content for this editable text widget */
		SLATE_ATTRIBUTE( UUTBBaseCommand*, Command )
		SLATE_EVENT( FOnObjectPropertyModified, OnObjectPropertyModified )
	
	SLATE_END_ARGS()


	SCommandDetailsView();
	~SCommandDetailsView();
	
	void Construct( const FArguments& InArgs );
	void SetObject(UUTBBaseCommand* Command);
	
private:
	TWeakObjectPtr<UUTBBaseCommand> Command;
	TSharedPtr<IDetailsView>		CommandDetailsView;
	FOnObjectPropertyModified		OnObjectPropertyModified;
};
