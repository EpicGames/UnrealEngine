// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "UTBBaseTab.h"
#include "UTBTabEditor.h"
#include "Widgets/Input/SButton.h"

/**
 * 
 */
class USERTOOLBOXCORE_API SButtonCommandWidget : public SButton
{

public:
	SLATE_BEGIN_ARGS( SButtonCommandWidget )
	:_Content()
	{
		
		
	}
	/** Slot for this button's content (optional) */
	SLATE_DEFAULT_SLOT( FArguments, Content )
	SLATE_EVENT( FOnClicked, OnClicked )
	
	SLATE_ATTRIBUTE( UUTBBaseCommand*, Command )
	
SLATE_END_ARGS()
	SButtonCommandWidget();
	~SButtonCommandWidget();

	void Construct(const FArguments& InArgs );
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	

	
private:
	TWeakObjectPtr<UUTBBaseCommand> Command;
	FOnContextMenuOpening OnContextMenuOpening;
	
};
class USERTOOLBOXCORE_API SCommandWidget : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnTabChangedNotify, UUserToolBoxBaseTab*)
	DECLARE_DELEGATE_ThreeParams(FOnCommandChanged, UUTBBaseCommand*, FString , int )
	DECLARE_DELEGATE_OneParam(FOnCommandSelected, UUTBBaseCommand*)
	DECLARE_DELEGATE_OneParam(FOnCommandDeleted,  UUTBBaseCommand* )
	DECLARE_DELEGATE_ThreeParams(FOnDropCommand, UUTBBaseCommand*, FString , int )

	SLATE_BEGIN_ARGS( SCommandWidget )
		: _Command() 
		, _IndexInSection(-1)
		, _SectionName("")
		{
		
		}

		/** Sets the text content for this editable text widget */
		SLATE_ATTRIBUTE( UUTBBaseCommand*, Command )
		SLATE_ATTRIBUTE( int, IndexInSection )
		SLATE_ATTRIBUTE( FString, SectionName )
		SLATE_ATTRIBUTE( TSharedPtr<FUTBTabEditor>, Editor )
		SLATE_EVENT(FOnTabChangedNotify, OnTabChanged)
		SLATE_EVENT(FOnCommandChanged, OnCommandChanged)
		SLATE_EVENT(FOnCommandSelected, OnCommandSelected)
		SLATE_EVENT(FOnCommandDeleted, OnCommandDeleted)
		SLATE_EVENT(FOnDropCommand, OnDropCommand)
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
	SLATE_END_ARGS()

	SCommandWidget();
	~SCommandWidget();
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	void Construct( const FArguments& InArgs );

	void OnObjectsReplacedCallback(const TMap<UObject*, UObject*>& ReplacementMap);
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	TSharedPtr<SWidget> CreateContextMenu();
private:
	void RegisterContextMenu();
	void RebuildWidget();
	TWeakPtr<FUTBTabEditor> Editor;
	FOnTabChangedNotify OnTabChanged;
	FOnCommandChanged OnCommandChanged;
	FOnCommandSelected OnCommandSelected;
	FOnCommandDeleted OnCommandDeleted;
	FOnDropCommand OnDropCommand;
	FOnContextMenuOpening OnContextMenuOpening;
	TWeakObjectPtr<UUTBBaseCommand> Command;
	int IndexInSection;
	FString SectionName;
};
