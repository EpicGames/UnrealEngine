// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IUTBTabEditor.h"
#include "EditorUndoClient.h"
#include "SCommandWithOptionsDetailsView.h"
#include "UTBBaseTab.h"
#include "Widgets/Views/SListView.h"
/**
 * 
 */

struct USERTOOLBOXCORE_API FCommandDragDropOperation : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FCommandDragDropOperation, FDragDropOperation);
	using FDragDropOperation::Construct;
	FCommandDragDropOperation();
	virtual ~FCommandDragDropOperation();

	static TSharedRef<FCommandDragDropOperation> New(UUTBBaseCommand* Command);

	
	TWeakObjectPtr<UUTBBaseCommand> Command;
	
};


class USERTOOLBOXCORE_API FUTBTabEditor : public IUTBTabEditor
		, public FEditorUndoClient
{
public:
	FUTBTabEditor();
	~FUTBTabEditor();
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCurrentSectionChange, FString);
	DECLARE_MULTICAST_DELEGATE(FOnSectionListChange);
	
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	virtual void InitUTBTabEditor(const EToolkitMode::Type Mode,const TSharedPtr<IToolkitHost>& InitToolkitHost, UUserToolBoxBaseTab* Tab);
	FString GetCurrentSection();

	FOnCurrentSectionChange	OnCurrentSectionChange;
	FOnSectionListChange OnSectionListChange;

	void AddCommandToCurrentSection(UUTBBaseCommand* InCommand);
	void RemoveCommandFromCurrentSection(UUTBBaseCommand* InCommand, bool ShouldRebuild=false);
	
	void AddCommandAt(UUTBBaseCommand* InCommand, FString Section,int Index);
	void SetCurrentCommandSelection(UUTBBaseCommand* Command);
	void NotifyTabHasChanged(UUserToolBoxBaseTab* Tab);
	void UpdateCommand(UUTBBaseCommand* Command);
	void UpdateCurrentSection();
	void CreateNewSection();
	void MoveSectionAfterExistingSection(FString SectionToMove,FString SectionToTarget);
	void RefreshSectionNames();
	void RemoveCurrentSection();
	void EditActiveSectionName();
	void RenameActiveSection(FString NewName);
	void BindCommands();
protected:

	void CreateOrRebuildUTBTabTabWidget();
	void CreateOrRebuildUTBTabSectionTabWidget();
	TSharedPtr<SWidget> CreateUTBTabDetailsTabWidget();
	TSharedPtr<SWidget> CreateUTBCommandTabWidget();
	TSharedPtr<SWidget> CreateUTBCommandDetailsTabWidget();

	void SetCurrentSection(FString SectionName);
	TArray<TSharedPtr<FString>> SectionNames;
	




	


	
	static const FName UTBTabTabId;
	static const FName UTBTabSectionTabId;
	static const FName UTBTabDetailsTabId;
	static const FName UTBCommandTabId;
	static const FName UTBCommandDetailsTabId;
	FString CurrentSection;
	FAssetData CurrentSelectedCommand;
	TSharedPtr<SVerticalBox>	UUTBTabTabWidget;
	TSharedPtr<SVerticalBox>    UUTBTabSectionTabWidget;
	TSharedPtr<SWidget>		    UUTBTabDetailsTabWidget;
	TSharedPtr<SWidget>		    UUTBCommandTabWidget;
	TSharedPtr<SWidget>		    UUTBCommandDetailsTabWidget;
	
	TSharedPtr<SCommandDetailsView>	CommandDetailsView;
	TSharedPtr<SVerticalBox> SectionVerticalBox;
	TSharedPtr<SListView<TSharedPtr<FString>>> SectionListView;
};

