// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataAsset.h"
#include "Engine/DataTable.h"
#include "IconsTracker.generated.h"

USTRUCT()
struct FIconInfo
{
	GENERATED_BODY()
	UPROPERTY(EditAnywhere , Category="Icon Tracker")
	FString Id;
	UPROPERTY(EditAnywhere , Category="Icon Tracker")
	FString Path;
	UPROPERTY(EditAnywhere , Category="Icon Tracker")
	FVector2D IconSize=FVector2D(30.0f);
	
	
};
USTRUCT()
struct FIconFolderInfo : public FTableRowBase
{
	GENERATED_BODY()
	UPROPERTY(EditAnywhere , Category="Icon Tracker")
	FDirectoryPath FolderPath;
	UPROPERTY(EditAnywhere , Category="Icon Tracker")
	FString PrefixId;
	UPROPERTY(EditAnywhere , Category="Icon Tracker")
	FVector2D IconSize=FVector2D(30.0f);
};

UCLASS()
class USERTOOLBOXCORE_API UIconsTracker : public UDataAsset
{
	GENERATED_BODY()

	public:
	UPROPERTY(EditAnywhere , Category="Icon Tracker")
	TArray<FIconFolderInfo>		IconFolderInfos;
	UPROPERTY(EditAnywhere , Category="Icon Tracker")
	FString					PrefixId;
	
};
