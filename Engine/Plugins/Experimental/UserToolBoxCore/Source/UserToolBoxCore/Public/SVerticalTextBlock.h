// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Text/STextBlock.h"
class SVerticalTextBlock : public STextBlock
{
public:

	SLATE_BEGIN_ARGS( SVerticalTextBlock )
		: _Text()
		, _TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "NormalText" ) )
		, _Font()
		, _StrikeBrush()
		, _ColorAndOpacity()
		, _ShadowOffset()
		, _ShadowColorAndOpacity()
		, _HighlightColor()
		, _HighlightShape()
		, _HighlightText()
		, _WrapTextAt(0.0f)
		, _AutoWrapText(false)
		, _WrappingPolicy(ETextWrappingPolicy::DefaultWrapping)
		, _TransformPolicy()
		, _Margin()
		, _LineHeightPercentage(1.0f)
		, _Justification(ETextJustify::Left)
		, _MinDesiredWidth(0.0f)
		, _TextShapingMethod()
		, _TextFlowDirection()
		, _LineBreakPolicy()
		, _OverflowPolicy()
		, _SimpleTextMode(false)
	{
		_Clipping = EWidgetClipping::OnDemand;
	}

	/** The text displayed in this text block */
	SLATE_ATTRIBUTE( FText, Text )

	/** Pointer to a style of the text block, which dictates the font, color, and shadow options. */
	SLATE_STYLE_ARGUMENT( FTextBlockStyle, TextStyle )

	/** Sets the font used to draw the text */
	SLATE_ATTRIBUTE( FSlateFontInfo, Font )

	/** Sets the brush used to strike through the text */
	SLATE_ATTRIBUTE( const FSlateBrush*, StrikeBrush )

	/** Text color and opacity */
	SLATE_ATTRIBUTE( FSlateColor, ColorAndOpacity )

	/** Drop shadow offset in pixels */
	SLATE_ATTRIBUTE( FVector2D, ShadowOffset )

	/** Shadow color and opacity */
	SLATE_ATTRIBUTE( FLinearColor, ShadowColorAndOpacity )

	/** The color used to highlight the specified text */
	SLATE_ATTRIBUTE( FLinearColor, HighlightColor )

	/** The brush used to highlight the specified text */
	SLATE_ATTRIBUTE( const FSlateBrush*, HighlightShape )

	/** Highlight this text in the text block */
	SLATE_ATTRIBUTE( FText, HighlightText )

	/** Whether text wraps onto a new line when it's length exceeds this width; if this value is zero or negative, no wrapping occurs. */
	SLATE_ATTRIBUTE( float, WrapTextAt )

	/** Whether to wrap text automatically based on the widget's computed horizontal space.  IMPORTANT: Using automatic wrapping can result
		in visual artifacts, as the the wrapped size will computed be at least one frame late!  Consider using WrapTextAt instead.  The initial 
		desired size will not be clamped.  This works best in cases where the text block's size is not affecting other widget's layout. */
	SLATE_ATTRIBUTE( bool, AutoWrapText )

	/** The wrapping policy to use */
	SLATE_ATTRIBUTE( ETextWrappingPolicy, WrappingPolicy )

	/** The transform policy to use */
	SLATE_ATTRIBUTE( ETextTransformPolicy, TransformPolicy )

	/** The amount of blank space left around the edges of text area. */
	SLATE_ATTRIBUTE( FMargin, Margin )

	/** The amount to scale each lines height by. */
	SLATE_ATTRIBUTE( float, LineHeightPercentage )

	/** How the text should be aligned with the margin. */
	SLATE_ATTRIBUTE( ETextJustify::Type, Justification )

	/** Minimum width that a text block should be */
	SLATE_ATTRIBUTE( float, MinDesiredWidth )

	/** Which text shaping method should we use? (unset to use the default returned by GetDefaultTextShapingMethod) */
	SLATE_ARGUMENT( TOptional<ETextShapingMethod>, TextShapingMethod )
		
	/** Which text flow direction should we use? (unset to use the default returned by GetDefaultTextFlowDirection) */
	SLATE_ARGUMENT( TOptional<ETextFlowDirection>, TextFlowDirection )

	/** The iterator to use to detect appropriate soft-wrapping points for lines (or null to use the default) */
	SLATE_ARGUMENT( TSharedPtr<IBreakIterator>, LineBreakPolicy )

	/** Determines what happens to text that is clipped and doesn't fit within the clip rect for this widget */
	SLATE_ARGUMENT(TOptional<ETextOverflowPolicy>, OverflowPolicy)

	/**
	 * If this is enabled, text shaping, wrapping, justification are disabled in favor of much faster text layout and measurement.
	 * This feature is suitable for numbers and text that changes often and impact performance.
	 * Enabling this setting may cause certain languages (such as Right to left languages) to not display properly.
	 */
	SLATE_ARGUMENT( bool, SimpleTextMode )

	/** Called when this text is double clicked */
	SLATE_EVENT(FPointerEventHandler, OnDoubleClicked)

SLATE_END_ARGS()

/** Constructor */
SVerticalTextBlock();

	/** Destructor */
	~SVerticalTextBlock();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct( const FArguments& InArgs );

	virtual FVector2D ComputeDesiredSize(const float) const override;
};