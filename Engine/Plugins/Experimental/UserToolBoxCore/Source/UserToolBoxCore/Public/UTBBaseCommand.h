// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/Widget.h"
#include "Framework/Commands/InputChord.h"
#include "UTBBaseCommand.generated.h"


/**
 * UUTBBaseCommand is the base class for any command of the usertoolbox framework.
 * Inherit directly from this class if you want to create a new command implemented in c++
 */
UCLASS(Blueprintable, Abstract,EditInlineNew)
class USERTOOLBOXCORE_API UUTBBaseCommand : public UObject
{
	GENERATED_BODY()
	public:
	/** The name of the command */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Command Description", AssetRegistrySearchable)
	FString Name="Default name";
	/** The icon path for the command */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Command Description")
	FString IconPath;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Command Description")
	FString Tooltip;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Command Description", AssetRegistrySearchable)
	FString Category="Default";
	UPROPERTY(EditAnywhere,BlueprintReadWrite, Category="Command Description")
	bool bShowParameters=false;
	
	UPROPERTY(EditAnywhere, Category="Command Description")
	FInputChord	KeyboardShortcut=FInputChord();

	UPROPERTY(EditAnywhere,BlueprintReadWrite,meta=(AllowAbstract = "false", MustImplement = "/Script/UserToolBoxCore.UTBUICommand" ), Category="Command Description")
	TObjectPtr<UClass>	UIOverride=nullptr;
	UPROPERTY(Transient)
	TObjectPtr<UObject>		UI=nullptr;
	UPROPERTY()
	bool bIsTransaction=false;
	virtual void Execute();
	UFUNCTION()
	void ExecuteCommand();
	virtual bool DisplayParameters();
	virtual UUTBBaseCommand* CopyCommand(UObject* Owner) const ;

	UFUNCTION(BlueprintCallable,Category="Transaction")
	void AddObjectToTransaction(UObject* Object);
	UFUNCTION(BlueprintCallable,Category="Transaction")
	void AddObjectsToTransaction(TArray<UObject*> Objects);
};

