// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseUICommandInterface.h"
#include "Engine/DataAsset.h"
#include "Widgets/SCompoundWidget.h"
#include "UTBBaseTab.generated.h"

/**
 * 
 */
class UUTBDefaultUITemplate;
class UUTBBaseCommand;


/**
 * UTBTabSection is the class that represent a section in a UUserToolBoxBaseTab
 * it contains the name of the section and the list of command attach to that section
 */
struct FUITemplateParameters
{
	bool	bForceHideSettingsButton=false;
	bool	bPrefixSectionWithTabName=false;
	bool	bPostfixSectionWithTabName=false;
};

UCLASS()
class USERTOOLBOXCORE_API UUTBTabSection : public UObject
{
	GENERATED_BODY()
public:
	/** The name of the section*/
	UPROPERTY(EditAnywhere, Category="Command Tab")
	FString								SectionName;
	/** The list of command */
	UPROPERTY()
	TArray<TObjectPtr<UUTBBaseCommand>>			Commands;
};
/**
 * UUserToolBoxBaseTab is the class that represent a configurable tab
 * This class contains the name of the tab and a section's list.
 * In that class, you can also specified an UTBUICommand to override "by default" the UI of each command.
 * Note: The override define in the command override the UI override define in the tab.
 * This is the class serialized as an asset.
 */
UCLASS()
class USERTOOLBOXCORE_API UUserToolBoxBaseTab : public UDataAsset
{
	GENERATED_BODY()
public:
	static const FName PlaceHolderSectionName;
	/** The name of the toolbox */
	UPROPERTY(EditAnywhere, Category="Command Tab", AssetRegistrySearchable)
	FString Name = "DefaultUserToolBox";
	/** The UI template to use to render this tab*/
	UPROPERTY(EditAnywhere, Category="Command Tab")
	TSubclassOf<UUTBDefaultUITemplate>	TabUI;
	/** the default override for each command ( does not override ui command specified in the command) */
	UPROPERTY(EditAnywhere, meta=(MustImplement = "/Script/UserToolBoxCore.UTBUICommand", AllowAbstract="false"), Category="Command Tab")
	TObjectPtr<UClass>	DefaultCommandUIOverride;


	/** Should the setting's icon be visible on the top right of the tab ( for easy access)*/
	UPROPERTY(EditAnywhere, Category="Command Tab")
	bool	IsSettingShouldBeVisible=true;
	UPROPERTY(EditAnywhere, Category="Command Tab",AssetRegistrySearchable)
	bool	bIsVisibleInViewportOverlay=false;
	UPROPERTY(EditAnywhere, Category="Command Tab",AssetRegistrySearchable)
	bool	bIsVisibleInDrawer=true;
	UPROPERTY(EditAnywhere, Category="Command Tab", AssetRegistrySearchable)
	bool bIsVisibleInWindowsMenu =true;
	/** Function to generate the raw widget of the tab. you may consider using the SUserToolBoxTabWidget instead if you want to register your different commands to the input manager etc..*/
	TSharedPtr<SWidget>  GenerateUI();
	TSharedPtr<SWidget>  GenerateUI(const TSubclassOf<UUTBDefaultUITemplate> TabUiClass,const FUITemplateParameters& Params);
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
	/** Insert a default command from the class at a specific location in a specific section
	 * @param InClass the class we want to instanciate
	 * @param Section the section name
	 * @param Index  the position in the section of the new command
	 */
	void InsertCommandFromClass(const TSubclassOf<UUTBBaseCommand> InClass,const  FString Section,const int Index=-1);
	/** Insert a command at a specific location in a specific section
	 * @param InCommand the command we want to add to the tab
	 * @param Section the section name
	 * @param Index the position in the section of the new command
	 */
	void InsertCommand(const UUTBBaseCommand* InCommand,const  FString SectionName,const int Index=-1);
	/** remove a command at a specific location in a specific section
	* @param Section the section name
	* @param Index the position in the section of the command to remove
	*/
	void RemoveCommand(const FString SectionName,const int Index=0);
	/** return the section with the specified name
	* @param SectionName the section name
	*/
	UUTBTabSection* GetSection(const FString SectionName);
	/** return the section with the specified name or create it if it does not exist
	* @param SectionName the section name
	*/
	UUTBTabSection* GetOrCreateSection(const FString SectionName);
	/** remove a section
	* @param Section The section to remove
	*/
	void RemoveSection( UUTBTabSection* Section);
	/** remove a section with a specific name
	* @param SectionName The name of the section to remove
	*/
	void RemoveSection(const FString SectionName);
	/** remove a command
	* @param Command The command to remove
	*/
	void RemoveCommand(UUTBBaseCommand* Command);
	/** remove a command contained in a specific section
	* @param Command The name of the section to remove
	* @param SectionName The name of the section to remove
	*/
	void RemoveCommandFromSection(UUTBBaseCommand* Command,const FString SectionName);
	/** replace a batch of command
	* @param ReplacementMap Map that contains the command to replace
	*/
	void ReplaceCommands(const TMap<UObject*,UObject*>& ReplacementMap);
	/** get all the section of that tab
	* @return an array of section 
	*/
	TArray<UUTBTabSection*>	GetSections();

	void MoveSectionAfterExistingSection(const FString SectionToMoveName,const FString SectionToTargetName);
	/** return the placeholder section
	* @return the placeholder section 
	*/
	UUTBTabSection* GetPlaceHolderSection();
	/** return the presence of the command in the tab
	* @return is the command inside that tab 
	*/
	bool ContainsCommand(const UUTBBaseCommand* Command) const;

	/** registering the command list
	*/
	void RegisterCommand();
	/** unregistering the command list
	*/
	void UnregisterCommand();

	void RemoveInvalidCommand();
private:
	// the array of sections
	UPROPERTY()
	TArray<TObjectPtr<UUTBTabSection>>		Sections;
	// the placeholder section
	UPROPERTY()
	TObjectPtr<UUTBTabSection>				PlaceholderSection;


	FDelegateHandle OnObjectReplacedHandle;
};

/**
 * SUserToolBoxTabWidget is the UI widget generated by a UUserToolBoxBaseTab.
 * It will generated the wanted UI based on the different UI override set in the UUserToolBoxBaseTab or in a specific command
 */
class USERTOOLBOXCORE_API SUserToolBoxTabWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SUserToolBoxTabWidget )
		: _Tab(nullptr),
		 _UIOverride(nullptr)

	{
		
	}
	/** Sets the text content for this editable text widget */
	SLATE_ATTRIBUTE( UUserToolBoxBaseTab*, Tab )
	SLATE_ATTRIBUTE( TSubclassOf<UUTBDefaultUITemplate>, UIOverride )
	SLATE_ATTRIBUTE( FUITemplateParameters, UIParameters )
SLATE_END_ARGS()

	
	~SUserToolBoxTabWidget();
	void Construct( const FArguments& InArgs );
	/*
	 * Function to request a tab UI update
	 * @param Tab Tab to update
	 */
	void UpdateTab(UUserToolBoxBaseTab* InTab);
	
	void OnObjectModified( UObject* Object, struct FPropertyChangedEvent&);
	
private:
	TObjectPtr<UUserToolBoxBaseTab> Tab;
	TSubclassOf<UUTBDefaultUITemplate> UIOverride;
	FUITemplateParameters Parameters;
	//Delegate handle
	FDelegateHandle		OnTabChangedDelegate;
	FDelegateHandle		OnObjectPropertyChangedDelegate;
	
};
