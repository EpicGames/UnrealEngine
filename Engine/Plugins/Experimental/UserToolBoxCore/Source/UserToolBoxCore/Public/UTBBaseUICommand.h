// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


#include "Blueprint/UserWidget.h"
#include "UTBBaseUICommandInterface.h"
#include "UTBBaseUICommand.generated.h"

class UUTBBaseCommand;
//Native base class for Command UI Customization
UCLASS()
class USERTOOLBOXCORE_API UBaseCommandNativeUI : public  UObject, public IUTBUICommand
{
	GENERATED_BODY()
	public:
	virtual bool IsSupportingCommandClass(TSubclassOf<UUTBBaseCommand> CommandClass) override;
	virtual void SetCommand(UUTBBaseCommand* Command) override ;
	virtual void ExecuteCurrentCommand() override ;
	virtual TSharedRef<SWidget> GetUI() override ;


	protected:
	UPROPERTY()
	TObjectPtr<UUTBBaseCommand>	MyCommand;

public:
	void ReplaceCommand(const TMap<UObject*,UObject*>& Map);
	virtual void BeginDestroy() override;
};

UCLASS()
class USERTOOLBOXCORE_API UUTBCommandUMGUI : public UUserWidget , public IUTBUICommand
{
	GENERATED_BODY()

	public:
	UFUNCTION(BlueprintImplementableEvent, Category=" Command UI")
	bool DoesSupportClass(TSubclassOf<UUTBBaseCommand> ObjectClass);

	UFUNCTION(BlueprintCallable, Category=" Command UI")
	void ExecuteCommand();

	UPROPERTY(EditAnywhere,BlueprintReadOnly, Category=" Command UI")
	TObjectPtr<UUTBBaseCommand> Command;



	virtual bool IsSupportingCommandClass(TSubclassOf<UUTBBaseCommand> CommandClass) override;
	virtual void SetCommand(UUTBBaseCommand* Command) override;
	virtual void ExecuteCurrentCommand() override;
	virtual TSharedRef<SWidget> GetUI() override;

	void ReplaceCommand(const TMap<UObject*,UObject*>& Map);
	virtual void BeginDestroy() override;
};
