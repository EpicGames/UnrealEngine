// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "Blueprint/UserWidget.h"
#include "UTBBaseUICommandInterface.generated.h"

class UUTBBaseCommand;


//Interface shared by the native UI and by the UMG one
UINTERFACE(MinimalAPI)
class UUTBUICommand : public UInterface
{
	GENERATED_BODY()
};

class USERTOOLBOXCORE_API IUTBUICommand
{
	GENERATED_BODY()
	public:
	virtual bool IsSupportingCommandClass(TSubclassOf<UUTBBaseCommand> CommandClass)=0;
	virtual void SetCommand(UUTBBaseCommand* Command)=0;
	virtual void ExecuteCurrentCommand()=0;
	virtual TSharedRef<SWidget> GetUI()=0;

};
