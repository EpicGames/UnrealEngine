// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseUICommandInterface.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "UObject/NoExportTypes.h"
#include "UTBBaseUITab.generated.h"

class UUTBUMGCommandUI;
class UUserToolBoxBaseTab;
class UUTBBaseCommand;
struct FUITemplateParameters;



/**
 * 
 */
UCLASS()
class USERTOOLBOXCORE_API UUTBDefaultUITemplate : public UObject
{
	GENERATED_BODY()
	public:
	virtual TSharedPtr<SWidget> BuildTabUI(UUserToolBoxBaseTab* Tab, const FUITemplateParameters& Params);
	
	

};
UCLASS()
class USERTOOLBOXCORE_API UUTBToolBarTabUI : public UUTBDefaultUITemplate
{
	GENERATED_BODY()
public:
	virtual TSharedPtr<SWidget> BuildTabUI(UUserToolBoxBaseTab* Tab, const FUITemplateParameters& Params) override;
	
protected:
	virtual TSharedPtr<FToolBarBuilder> GetBuilder(TSharedPtr<const FUICommandList> InCommandList, FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender = nullptr, const bool InForceSmallIcons = false);

};
UCLASS()
class USERTOOLBOXCORE_API UUTBPaletteTabUI : public UUTBToolBarTabUI
{
	GENERATED_BODY()
protected:
	virtual TSharedPtr<FToolBarBuilder> GetBuilder(TSharedPtr<const FUICommandList> InCommandList, FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender = nullptr, const bool InForceSmallIcons = false)override;

};
UCLASS()
class USERTOOLBOXCORE_API UUTBVerticalToolBarTabUI : public UUTBToolBarTabUI
{
	GENERATED_BODY()
protected:
	virtual TSharedPtr<FToolBarBuilder> GetBuilder(TSharedPtr<const FUICommandList> InCommandList, FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender = nullptr, const bool InForceSmallIcons = false)override;

};
UCLASS()
class USERTOOLBOXCORE_API UUTBSlimHorizontalToolBarTabUI : public UUTBToolBarTabUI
{
	GENERATED_BODY()
protected:
	virtual TSharedPtr<FToolBarBuilder> GetBuilder(TSharedPtr<const FUICommandList> InCommandList, FMultiBoxCustomization InCustomization, TSharedPtr<FExtender> InExtender = nullptr, const bool InForceSmallIcons = false)override;

};