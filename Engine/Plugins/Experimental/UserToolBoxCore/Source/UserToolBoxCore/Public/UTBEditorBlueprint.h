// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUtilityBlueprint.h"
#include "UTBEditorBlueprint.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXCORE_API UUTBEditorBlueprint : public UEditorUtilityBlueprint
{
	GENERATED_BODY()
	
};
