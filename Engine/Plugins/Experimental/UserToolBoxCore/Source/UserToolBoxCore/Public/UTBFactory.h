// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "UTBFactory.generated.h"

/**
 * 
 */
UCLASS()
class USERTOOLBOXCORE_API UUTBEditorUtilityBlueprintFactory : public UFactory
{
	GENERATED_UCLASS_BODY()
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;
};

UCLASS()
class USERTOOLBOXCORE_API UUTBCommandFactory : public UFactory
{
	GENERATED_UCLASS_BODY()
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;
	virtual bool ConfigureProperties() override;
	
	private:
	UClass* ParentClass;
};
UCLASS()
class USERTOOLBOXCORE_API UUTBCommandTabFactory : public UFactory
{
	GENERATED_UCLASS_BODY()
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;

};
UCLASS()
class USERTOOLBOXCORE_API UIconTrackerFactory : public UFactory
{
	GENERATED_UCLASS_BODY()
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;

};
