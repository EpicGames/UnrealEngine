// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UTBBaseCommand.h"
#include "UObject/NoExportTypes.h"
#include "UserToolBoxBaseBlueprint.generated.h"

/**
 * 
 */


UCLASS(Abstract, hideCategories=(Object),Blueprintable, meta = (ShowWorldContextPin))
class USERTOOLBOXCORE_API UUserToolBoxBaseBlueprint : public UUTBBaseCommand
{
	GENERATED_BODY()
	public:


	UFUNCTION(BlueprintImplementableEvent)
	void Command() ;

	virtual void Execute() override;
};


