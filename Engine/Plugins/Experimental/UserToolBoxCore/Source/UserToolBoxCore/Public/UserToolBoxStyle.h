// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
 	

#include "Styling/ISlateStyle.h"
#include "IconsTracker.h"
struct FSlateImageBrush;

/**  */
class USERTOOLBOXCORE_API FUserToolBoxStyle
{
public:

	static void Initialize();

	static void Shutdown();

	/** reloads textures used by slate renderer */
	static void ReloadTextures();

	/** @return The Slate style set for the Shooter game */
	static const ISlateStyle& Get();

	static FName GetStyleSetName();
	static void AddExternalImageBrushes(const TArray<FIconInfo>& IconInfos);
	static TArray<FString> GetAvailableExternalImageBrushes();
private:

	static TSharedRef< class FSlateStyleSet > Create();



	static TSharedPtr< class FSlateStyleSet > StyleInstance;
	static TArray<FString> ExternalBrushIds;
	static TMap<FString, TUniquePtr<FSlateImageBrush>> ExternalBrushes;
};