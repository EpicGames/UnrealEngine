// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeCategories.h"
#include "EditorSubsystem.h"
#include "UTBBaseTab.h"
#include "UTBBaseUITab.h"
#include "Widgets/Docking/SDockTab.h"
#include "UserToolBoxSubsystem.generated.h"

class UUTBCommandUMGUI;
class UUTBBaseCommand;
class UUTBBaseUITab;
/**
 * 
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTabChanged, UUserToolBoxBaseTab*)
UCLASS(BlueprintType)
class USERTOOLBOXCORE_API UUserToolboxSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()
	UFUNCTION(BlueprintCallable, Category="User Toolbox Subsystem")
	void RegisterTabData();

	

	public:
	TArray<FAssetData> GetAvailableTabList();
	FOnTabChanged		OnTabChanged;

	UFUNCTION(BlueprintCallable,Category="User Toolbox")
	bool PickAnIcon(FString& OutValue);
	
	private:
	UPROPERTY()
	TMap<FString,TObjectPtr<UUserToolBoxBaseTab>>	RegisteredTabs;
	
	
	public:
	UFUNCTION(BlueprintCallable,Category="User Toolbox")
	void RefreshIcons();

	
	
	TSharedPtr<SWidget> GenerateTabUI(const FAssetData Data, TSubclassOf<UUTBDefaultUITemplate> Ui=nullptr,const FUITemplateParameters& Parameters=FUITemplateParameters());

	
	/** Implement this for initialization of instances of the system */
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	/** Implement this for deinitialization of instances of the system */
	virtual void Deinitialize() override;
	
	TSharedRef<SDockTab> CreateTab(const FSpawnTabArgs& SpawnTabArgs);
	void ReinitializeUIs();
	
	static EAssetTypeCategories::Type AssetCategory;
	private:

	TArray<TPair<FString,FString>> Icons;

	TArray<TSharedPtr<FString>>		IconOptions;

public:
	void RegisterDrawer();
	void UpdateLevelViewportWidget();
private:

	void OnAssetChanged(const FAssetData& InAssetData);

	// Delegate handle
	FDelegateHandle OnFileLoadHandle;

	TSharedPtr<SWidget>	LevelViewportOverlayWdget;
	TArray<FName>		RegisteredDrawer;
	TArray<FAssetData> AvailableTabList;
};
