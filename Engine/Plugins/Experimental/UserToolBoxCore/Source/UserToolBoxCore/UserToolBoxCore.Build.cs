// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UserToolBoxCore : ModuleRules
{
	public UserToolBoxCore(ReadOnlyTargetRules Target) : base(Target)
	{
		
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"EditorSubsystem",
				"Blutility",
				"ToolWidgets",
				"AssetTools",
				"PropertyEditor",
				"UMG",
				"UMGEditor",
				"ToolMenus",
				"Projects",
				"LevelEditor",
				"Kismet",
				"InputCore",
				"EditorStyle",
				"EditorInteractiveToolsFramework",
				"InteractiveToolsFramework",
				"EditorWidgets", "SlateScriptingCommands"
				,"StatusBar"
				// ... add private dependencies that you statically link with here ...	
			}
			);
		if (Target.Version.MajorVersion >= 5)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"EditorFramework"
				});
		}
		else
		{
		}


		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
