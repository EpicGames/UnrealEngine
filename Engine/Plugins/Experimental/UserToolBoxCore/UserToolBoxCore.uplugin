{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.10",
	"FriendlyName": "UserToolBoxCore",
	"Description": "Core functionnality to create custom editor tab",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "UserToolBoxCore",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins":
	[
		{
			"Name": "SlateScripting",
			"Enabled": true
		}
	]
}